// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioCaptureAndroid : ModuleRules
{
    public AudioCaptureAndroid(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.Add("Core");
        PrivateDependencyModuleNames.Add("GoogleOboe");
        PublicIncludePathModuleNames.Add("GoogleOboe");
        AddEngineThirdPartyPrivateStaticDependencies(Target, "GoogleOboe");
        PrivateDependencyModuleNames.Add("AudioCaptureCore");
        PublicDefinitions.Add("WITH_AUDIOCAPTURE=1");
    }
}
