// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureAndroid.h"

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAudioCaptureAndroid, Log, All);
DEFINE_LOG_CATEGORY(LogAudioCaptureAndroid);

Audio::FAudioCaptureAndroidStream::FAudioCaptureAndroidStream()
	: NumChannels(1)
	, SampleRate(48000)
{
}

oboe::DataCallbackResult Audio::FAudioCaptureAndroidStream::onAudioReady(oboe::AudioStream *oboeStream, void *audioData, int32 numFrames)
{
	OnAudioCapture(audioData, numFrames, 0.0, false);
	return oboe::DataCallbackResult::Continue;
}

bool Audio::FAudioCaptureAndroidStream::GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex)
{
	// TODO: see if we can query the device for this, otherwise use these defaults
	OutInfo.DeviceName = TEXT("Default Android Audio Device");
	OutInfo.InputChannels = NumChannels;
	OutInfo.PreferredSampleRate = SampleRate;
	return true;
}

bool Audio::FAudioCaptureAndroidStream::OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired)
{
	// Build stream settings object.
	oboe::AudioStreamBuilder StreamBuilder;
	StreamBuilder.setDeviceId(0);
	StreamBuilder.setCallback(this);
	StreamBuilder.setDirection(oboe::Direction::Input);
	StreamBuilder.setSampleRate(SampleRate);
	StreamBuilder.setChannelCount(NumChannels);
	StreamBuilder.setFormat(oboe::AudioFormat::Float);

	// Open up a capture stream
	oboe::AudioStream* NewStream;
	oboe::Result Result = StreamBuilder.openStream(&NewStream);

	bool bSuccess = Result == oboe::Result::OK;

	InputOboeStream.Reset(NewStream);

	OnCapture = MoveTemp(InOnCapture);

	if (!bSuccess)
	{
		// Log error on failure.
		FString ErrorString(UTF8_TO_TCHAR(convertToText(Result)));
		UE_LOGF(LogAudioCaptureAndroid, Error, "Failed to open oboe capture stream: %ls", *ErrorString)
	}

	return bSuccess;
}

bool Audio::FAudioCaptureAndroidStream::CloseStream()
{
	if (InputOboeStream)
	{
		InputOboeStream->close();
		InputOboeStream.Reset();
	}

	return true;
}

bool Audio::FAudioCaptureAndroidStream::StartStream()
{
	if (!InputOboeStream)
	{
		return false;
	}

	return InputOboeStream->requestStart() == oboe::Result::OK;
}

bool Audio::FAudioCaptureAndroidStream::StopStream()
{
	if (!InputOboeStream)
	{
		return false;
	}

	return InputOboeStream->stop() == oboe::Result::OK;
}

bool Audio::FAudioCaptureAndroidStream::AbortStream()
{
	return CloseStream();
}

bool Audio::FAudioCaptureAndroidStream::GetStreamTime(double& OutStreamTime)
{
	if (!InputOboeStream)
	{
		return false;
	}

	int64 FramesRead = InputOboeStream->getFramesRead();
	const oboe::ResultWithValue<oboe::FrameTimestamp> Timestamp = InputOboeStream->getTimestamp(CLOCK_MONOTONIC);;
	int64 OutTimestampNanoseconds = Timestamp.value().timestamp;
	OutStreamTime = ((double)OutTimestampNanoseconds) / oboe::kNanosPerSecond;
	return Timestamp.error() == oboe::Result::OK;
}

bool Audio::FAudioCaptureAndroidStream::IsStreamOpen() const
{
	return InputOboeStream != nullptr;
}

bool Audio::FAudioCaptureAndroidStream::IsCapturing() const
{
	if (InputOboeStream)
	{
		return InputOboeStream->getState() == oboe::StreamState::Started;
	}
	else
	{
		return false;
	}
}

void Audio::FAudioCaptureAndroidStream::OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow)
{
	const float* FloatBuffer = static_cast<float*>(InBuffer);
	OnCapture(FloatBuffer, InBufferFrames, NumChannels, SampleRate, StreamTime, bOverflow);
}

bool Audio::FAudioCaptureAndroidStream::GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices)
{
	// TODO: Add individual devices for different ports here.
	OutDevices.Reset();

	FCaptureDeviceInfo& DeviceInfo = OutDevices.AddDefaulted_GetRef();
	GetCaptureDeviceInfo(DeviceInfo, 0);

	return true;
}
