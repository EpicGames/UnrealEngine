// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioCaptureCore.h"
#include "oboe/Oboe.h"

namespace Audio
{
	class FAudioCaptureAndroidStream : public IAudioCaptureStream, public oboe::AudioStreamCallback
	{
	public:
		FAudioCaptureAndroidStream();

		virtual bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex) override;
		virtual bool OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired) override;
		virtual bool CloseStream() override;
		virtual bool StartStream() override;
		virtual bool StopStream() override;
		virtual bool AbortStream() override;
		virtual bool GetStreamTime(double& OutStreamTime) override;
		virtual int32 GetSampleRate() const override { return SampleRate; }
		virtual bool IsStreamOpen() const override;
		virtual bool IsCapturing() const override;
		virtual void OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow) override;
		virtual bool GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices) override;

		// begin oboe::AudioStreamCallback
		virtual oboe::DataCallbackResult onAudioReady(oboe::AudioStream *oboeStream, void *audioData, int32 numFrames) override;
		// end oboe::AudioStreamCallback

	private:
		FOnAudioCaptureFunction OnCapture;
		int32 NumChannels;
		int32 SampleRate;

		TUniquePtr<oboe::AudioStream> InputOboeStream;
	};
}