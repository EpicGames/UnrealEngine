// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioCaptureRtAudio : ModuleRules
{
	protected virtual bool WithRtAudio { get => false; }

	public AudioCaptureRtAudio(ReadOnlyTargetRules Target) : base(Target)
	{
		// RtAudio throws exceptions for errors.
		bEnableExceptions = true;
		// AutoRTFM cannot be used with exceptions.
		bDisableAutoRTFMInstrumentation = true;

		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("AudioCaptureCore");

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PublicDefinitions.Add("WITH_RTAUDIO=1");
			PublicDefinitions.Add("WITH_AUDIOCAPTURE=1");

			// Allow us to use direct sound
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DirectSound");
		}
		else if(Target.Platform == UnrealTargetPlatform.Mac)
        {
			bRequiresPlatformSDK = true;
            PublicFrameworks.AddRange(new string[] { "CoreAudio", "AudioUnit", "AudioToolbox" });
            PublicDefinitions.Add("WITH_RTAUDIO=1");
            PublicDefinitions.Add("WITH_AUDIOCAPTURE=1");
        }
		else
		{
			PublicDefinitions.Add($"WITH_RTAUDIO={(WithRtAudio?"1":"0")}");
		}
	}
}
