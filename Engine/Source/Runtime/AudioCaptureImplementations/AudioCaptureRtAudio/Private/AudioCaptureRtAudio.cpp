// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureRtAudio.h"
#include "AudioCaptureCoreLog.h"
#include "AudioCaptureDeviceInterface.h"


static constexpr uint32 InvalidDeviceID = INDEX_NONE;

#if WITH_RTAUDIO
bool GetAudioFormatFromEncoding(Audio::EPCMAudioEncoding InEncoding, RtAudioFormat& OutAudioFormat)
{
	switch (InEncoding)
	{
	case Audio::EPCMAudioEncoding::PCM_8:
		OutAudioFormat = RTAUDIO_SINT8;
		break;

	case Audio::EPCMAudioEncoding::PCM_16:
		OutAudioFormat = RTAUDIO_SINT16;
		break;

	case Audio::EPCMAudioEncoding::PCM_24:
		OutAudioFormat = RTAUDIO_SINT24;
		break;

	case Audio::EPCMAudioEncoding::PCM_32:
		OutAudioFormat = RTAUDIO_SINT32;
		break;

		// Default to 32-bit float
	case Audio::EPCMAudioEncoding::UNKNOWN:
	case Audio::EPCMAudioEncoding::FLOATING_POINT_32:
		OutAudioFormat = RTAUDIO_FLOAT32;
		break;

	case Audio::EPCMAudioEncoding::FLOATING_POINT_64:
		OutAudioFormat = RTAUDIO_FLOAT64;
		break;

		// RtAudio does not support 24-bit audio in a 32-bit container
	case Audio::EPCMAudioEncoding::PCM_24_IN_32:
	default:
		return false;
	}

	return true;
}
#endif // WITH_RTAUDIO

Audio::FAudioCaptureRtAudioStream::FAudioCaptureRtAudioStream()
	: NumChannels(0)
	, SampleRate(0)
{
}

#if WITH_RTAUDIO
static int32 OnAudioCaptureCallback(void *OutBuffer, void* InBuffer, uint32 InBufferFrames, double StreamTime, RtAudioStreamStatus AudioStreamStatus, void* InUserData)
{
	Audio::FAudioCaptureRtAudioStream* AudioCapture = (Audio::FAudioCaptureRtAudioStream*) InUserData;

	AudioCapture->OnAudioCapture(InBuffer, InBufferFrames, StreamTime, AudioStreamStatus == RTAUDIO_INPUT_OVERFLOW);
	return 0;
}
#endif // WITH_RTAUDIO


bool Audio::FAudioCaptureRtAudioStream::GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex)
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("get device info");
	try
	{
		uint32 InputDeviceId = GetDefaultInputDevice();
		if (DeviceIndex != DefaultDeviceIndex)
		{
			InputDeviceId = DeviceIndex;
		}

		RtAudio::DeviceInfo DeviceInfo = CaptureDevice.getDeviceInfo(InputDeviceId);
		
		OutInfo.DeviceName = FString(FUTF8ToTCHAR(DeviceInfo.name.c_str()));
		OutInfo.DeviceId = FString(FUTF8ToTCHAR(DeviceInfo.deviceId.c_str()));
		OutInfo.InputChannels = DeviceInfo.inputChannels;
		OutInfo.PreferredSampleRate = DeviceInfo.preferredSampleRate;
        
        if (OutInfo.DeviceId.IsEmpty())
        {
            // Some platforms (such as Mac) don't use string identifiers so synthesize one from
			// the device name and index
            OutInfo.DeviceId = FString::Printf(TEXT("%s_%d"), *OutInfo.DeviceName, InputDeviceId);
        }
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		return false;
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		return false;
	}

	return true;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired)
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("open RtAudio stream");
	try
	{
		uint32 InputDeviceId = GetDefaultInputDevice();
		if (InParams.DeviceIndex != DefaultDeviceIndex)
		{
			// InParams.DeviceIndex is the index inside the sublist of devices including only
			// the Input devices.
			uint32 NumInputDevices = 0;
			bool DeviceFound = false;
			uint32 NumDevices = CaptureDevice.getDeviceCount();
			for (uint32 DeviceIndex = 0; DeviceIndex < NumDevices; DeviceIndex++)
			{
				FCaptureDeviceInfo DeviceInfo;
				GetCaptureDeviceInfo(DeviceInfo, DeviceIndex);
				bool IsInput = DeviceInfo.InputChannels > 0;
				if (!IsInput)
				{
					continue;
				}
				if (NumInputDevices == InParams.DeviceIndex)
				{
					DeviceFound = true;
					InputDeviceId = DeviceIndex;
					break;
				}
				NumInputDevices++;
			}
			if (!DeviceFound)
			{
				return false;
			}
		}

		RtAudio::DeviceInfo DeviceInfo = CaptureDevice.getDeviceInfo(InputDeviceId);

		RtAudio::StreamParameters RtAudioStreamParams;
		RtAudioStreamParams.deviceId = InputDeviceId;
		RtAudioStreamParams.firstChannel = 0;

		if (InParams.NumInputChannels == InvalidDeviceChannelCount)
		{
			RtAudioStreamParams.nChannels = FMath::Min((int32)DeviceInfo.inputChannels, 8);
		}
		else
		{
			RtAudioStreamParams.nChannels = InParams.NumInputChannels;
		}

		if (CaptureDevice.isStreamOpen())
		{
			CaptureDevice.stopStream();
			CaptureDevice.closeStream();
		}

		uint32 NumFrames = NumFramesDesired;
		NumChannels = RtAudioStreamParams.nChannels;
		OnCapture = MoveTemp(InOnCapture);

		if (InParams.SampleRate == InvalidDeviceSampleRate)
		{
			SampleRate = DeviceInfo.preferredSampleRate;
		}
		else
		{
			SampleRate = InParams.SampleRate;
		}

		RtAudioFormat AudioFormat;
		if (GetAudioFormatFromEncoding(InParams.PCMAudioEncoding, AudioFormat))
		{
			// Open up new audio stream
			CaptureDevice.openStream(nullptr, &RtAudioStreamParams, AudioFormat, SampleRate, &NumFrames, &OnAudioCaptureCallback, this);
			if (!CaptureDevice.isStreamOpen())
			{
				return false;
			}
		}
		else
		{
			return false;
		}

		SampleRate = CaptureDevice.getStreamSampleRate();
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		return false;
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		return false;
	}

	return true;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::CloseStream()
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("close RtAudio stream");
	try
	{
		if (CaptureDevice.isStreamOpen())
		{
			CaptureDevice.closeStream();
			return true;
		}
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::StartStream()
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("start RtAudio stream");
	try
	{
		CaptureDevice.startStream();
		return true;
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::StopStream()
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("stop RtAudio stream");
	try
	{
		if (CaptureDevice.isStreamOpen())
		{
			CaptureDevice.stopStream();
			return true;
		}
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::AbortStream()
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("abort RtAudio stream");
	try
	{
		if (CaptureDevice.isStreamOpen())
		{
			CaptureDevice.abortStream();
			return true;
		}
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::GetStreamTime(double& OutStreamTime)
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("get RtAudio stream time");
	try
	{
		OutStreamTime = CaptureDevice.getStreamTime();
		return true;
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::IsStreamOpen() const
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("check if RtAudio stream is open");
	try
	{
		return CaptureDevice.isStreamOpen();
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::IsCapturing() const
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("check if RtAudio stream is running");
	try
	{
		return CaptureDevice.isStreamRunning();
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

void Audio::FAudioCaptureRtAudioStream::OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow)
{
#if WITH_RTAUDIO
	OnCapture(InBuffer, InBufferFrames, NumChannels, SampleRate, StreamTime, bOverflow);
#endif // WITH_RTAUDIO
}

bool Audio::FAudioCaptureRtAudioStream::GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices)
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("get input devices available RtAudio");
	try
	{
		uint32 NumDevices = CaptureDevice.getDeviceCount();
		OutDevices.Reset();
		// Iterate over all devices and include in the output only the ones that are input devices
		for (uint32 DeviceIndex = 0; DeviceIndex < NumDevices; DeviceIndex++)
		{
			FCaptureDeviceInfo DeviceInfo;
			GetCaptureDeviceInfo(DeviceInfo, DeviceIndex);
			bool IsInput = DeviceInfo.InputChannels > 0;
			if (IsInput)
			{
				OutDevices.Add(DeviceInfo);
			}
		}

		return true;
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return false;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return false;
#endif // !WITH_RTAUDIO
}

uint32 Audio::FAudioCaptureRtAudioStream::GetDefaultInputDevice()
{
#if WITH_RTAUDIO
	static const TCHAR* Action = TEXT("get default input device");
	try
	{
		return CaptureDevice.getDefaultInputDevice();
	}
	catch (const std::exception& e)
	{
		FString Message(e.what());
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
	}
	catch (...)
	{
		UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
	}

	return InvalidDeviceID;
#else // ^^^ WITH_RTAUDIO vvv !WITH_RTAUDIO
	return InvalidDeviceID;
#endif // !WITH_RTAUDIO
}

