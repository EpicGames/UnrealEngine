// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioCaptureCore.h"

#if PLATFORM_MICROSOFT
#include "Windows/WindowsHWrapper.h"
#endif

#ifndef WITH_RTAUDIO
#define WITH_RTAUDIO 0
#endif

#if WITH_RTAUDIO
THIRD_PARTY_INCLUDES_START
#include "RtAudio.h"
THIRD_PARTY_INCLUDES_END
#endif

namespace Audio
{
	class FAudioCaptureRtAudioStream : public IAudioCaptureStream
	{
	public:
		FAudioCaptureRtAudioStream();

		virtual bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex) override;
		virtual bool OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired) override;
		virtual bool CloseStream() override;
		virtual bool StartStream() override;
		virtual bool StopStream() override;
		virtual bool AbortStream() override;
		virtual bool GetStreamTime(double& OutStreamTime) override;
		virtual int32 GetSampleRate() const override { return SampleRate; }
		virtual bool IsStreamOpen() const override;
		virtual bool IsCapturing() const override;
		virtual void OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow) override;
		virtual bool GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices) override;

	private:
		FOnAudioCaptureFunction OnCapture;
		int32 NumChannels;
		int32 SampleRate;

		uint32 GetDefaultInputDevice();

#if WITH_RTAUDIO
		RtAudio CaptureDevice;
#endif
	};
}
