// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AudioCaptureDeviceInterface.h"
#include "AudioCaptureRtAudio.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

namespace Audio
{

	class FAudioCaptureRtAudioFactory : public IAudioCaptureFactory
	{
	public:
		virtual TUniquePtr<IAudioCaptureStream> CreateNewAudioCaptureStream() override
		{
			return TUniquePtr<IAudioCaptureStream>(new FAudioCaptureRtAudioStream());
		}
	};

	class FAudioCaptureRtAudioModule : public IModuleInterface
	{
	private:
		FAudioCaptureRtAudioFactory AudioCaptureFactory;

	public:
		virtual void StartupModule() override
		{
			IModularFeatures::Get().RegisterModularFeature(IAudioCaptureFactory::GetModularFeatureName(), &AudioCaptureFactory);
		}

		virtual void ShutdownModule() override
		{
			IModularFeatures::Get().UnregisterModularFeature(IAudioCaptureFactory::GetModularFeatureName(), &AudioCaptureFactory);
		}
	};
}

IMPLEMENT_MODULE(Audio::FAudioCaptureRtAudioModule, AudioCaptureRtAudio)
