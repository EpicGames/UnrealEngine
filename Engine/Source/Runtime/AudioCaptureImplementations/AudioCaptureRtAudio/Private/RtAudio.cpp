// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtAudio.h"
#include "AudioCaptureCoreLog.h"

#if WITH_RTAUDIO

#if PLATFORM_MICROSOFT || PLATFORM_MAC
/************************************************************************/
/*! \class RtAudio
    \brief Realtime audio i/o C++ classes.

    RtAudio provides a common API (Application Programming Interface)
    for realtime audio input/output across Linux (native ALSA, Jack,
    and OSS), Macintosh OS X (CoreAudio and Jack), and Windows
    (DirectSound, ASIO and WASAPI) operating systems.

    RtAudio WWW site: http://www.music.mcgill.ca/~gary/rtaudio/

    RtAudio: realtime audio i/o C++ classes
    Copyright (c) 2001-2017 Gary P. Scavone

    Permission is hereby granted, free of charge, to any person
    obtaining a copy of this software and associated documentation files
    (the "Software"), to deal in the Software without restriction,
    including without limitation the rights to use, copy, modify, merge,
    publish, distribute, sublicense, and/or sell copies of the Software,
    and to permit persons to whom the Software is furnished to do so,
    subject to the following conditions:

    The above copyright notice and this permission notice shall be
    included in all copies or substantial portions of the Software.

    Any person wishing to distribute modifications to the Software is
    asked to send the modifications to the original developer so that
    they can be incorporated into the canonical version.  This is,
    however, not a binding provision of this license.

    THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND,
    EXPRESS OR IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF
    MERCHANTABILITY, FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT.
    IN NO EVENT SHALL THE AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR
    ANY CLAIM, DAMAGES OR OTHER LIABILITY, WHETHER IN AN ACTION OF
    CONTRACT, TORT OR OTHERWISE, ARISING FROM, OUT OF OR IN CONNECTION
    WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE SOFTWARE.
*/
/************************************************************************/

// RtAudio: Version 5.0.0

#include "AudioCaptureCore.h"
THIRD_PARTY_INCLUDES_START
#include <iostream>
#include <cstdlib>
#include <cstring>
#include <climits>
#include <cmath>
#include <algorithm>
THIRD_PARTY_INCLUDES_END

// Static variable definitions.
const unsigned int RtApi::MAX_SAMPLE_RATES = 14;
const unsigned int RtApi::SAMPLE_RATES[] = {
  4000, 5512, 8000, 9600, 11025, 16000, 22050,
  32000, 44100, 48000, 88200, 96000, 176400, 192000
};

#if defined(__WINDOWS_DS__) || defined(__WINDOWS_ASIO__) || defined(__WINDOWS_WASAPI__)

#include "Windows/AllowWindowsPlatformTypes.h"
#pragma warning ( disable : 6387 )

#define DISABLE_THROW 1

#if DISABLE_THROW
static void throw_wrapper(const RtAudioError& Error)
{
	Error.printMessage();
}
#else 
#define throw_wrapper throw
#endif

  #define MUTEX_INITIALIZE(A) InitializeCriticalSection(A)
  #define MUTEX_DESTROY(A)    DeleteCriticalSection(A)
  #define MUTEX_LOCK(A)       EnterCriticalSection(A)
  #define MUTEX_UNLOCK(A)     LeaveCriticalSection(A)

  #include "tchar.h"

  static std::string convertCharPointerToStdString(const char *text)
  {
    return std::string(text);
  }

  static std::string convertCharPointerToStdString(const wchar_t *text)
  {
    int length = WideCharToMultiByte(CP_UTF8, 0, text, -1, NULL, 0, NULL, NULL);
    std::string s( length-1, '\0' );
    WideCharToMultiByte(CP_UTF8, 0, text, -1, &s[0], length, NULL, NULL);
    return s;
  }

  static std::string convertGuidToStdString(LPGUID guid)
  {
	char result[39]; // 32 hex chars + 4 hyphens + 2 braces + null
	if (guid)
	{
		snprintf(
			result, sizeof(result), "{%08lx-%04x-%04x-%02x%02x-%02x%02x%02x%02x%02x%02x}",
			guid->Data1, guid->Data2, guid->Data3,
			guid->Data4[0], guid->Data4[1], guid->Data4[2], guid->Data4[3],
			guid->Data4[4], guid->Data4[5], guid->Data4[6], guid->Data4[7]);
	}
	else
	{
		result[0] = '\0';
	}

	return result;
  }

#elif defined(__LINUX_ALSA__) || defined(__LINUX_PULSE__) || defined(__UNIX_JACK__) || defined(__LINUX_OSS__) || defined(__MACOSX_CORE__)
  // pthread API
  #define MUTEX_INITIALIZE(A) pthread_mutex_init(A, NULL)
  #define MUTEX_DESTROY(A)    pthread_mutex_destroy(A)
  #define MUTEX_LOCK(A)       pthread_mutex_lock(A)
  #define MUTEX_UNLOCK(A)     pthread_mutex_unlock(A)
#else
  #define MUTEX_INITIALIZE(A) abs(*A) // dummy definitions
  #define MUTEX_DESTROY(A)    abs(*A) // dummy definitions
#endif

// *************************************************** //
//
// RtAudio definitions.
//
// *************************************************** //

std::string RtAudio :: getVersion( void )
{
  return RTAUDIO_VERSION;
}

void RtAudio :: getCompiledApi( std::vector<RtAudio::Api> &apis )
{
  apis.clear();

  // The order here will control the order of RtAudio's API search in
  // the constructor.
#if defined(__UNIX_JACK__)
  apis.push_back( UNIX_JACK );
#endif
#if defined(__LINUX_ALSA__)
  apis.push_back( LINUX_ALSA );
#endif
#if defined(__LINUX_PULSE__)
  apis.push_back( LINUX_PULSE );
#endif
#if defined(__LINUX_OSS__)
  apis.push_back( LINUX_OSS );
#endif
#if defined(__WINDOWS_ASIO__)
  apis.push_back( WINDOWS_ASIO );
#endif
#if defined(__WINDOWS_WASAPI__)
  apis.push_back( WINDOWS_WASAPI );
#endif
#if defined(__WINDOWS_DS__)
  apis.push_back( WINDOWS_DS );
#endif
#if defined(__MACOSX_CORE__)
  apis.push_back( MACOSX_CORE );
#endif
#if defined(__RTAUDIO_DUMMY__)
  apis.push_back( RTAUDIO_DUMMY );
#endif
}

void RtAudio :: openRtApi( RtAudio::Api api )
{
  if ( rtapi_ )
    delete rtapi_;
  rtapi_ = 0;

#if defined(__UNIX_JACK__)
  if ( api == UNIX_JACK )
    rtapi_ = new RtApiJack();
#endif
#if defined(__LINUX_ALSA__)
  if ( api == LINUX_ALSA )
    rtapi_ = new RtApiAlsa();
#endif
#if defined(__LINUX_PULSE__)
  if ( api == LINUX_PULSE )
    rtapi_ = new RtApiPulse();
#endif
#if defined(__LINUX_OSS__)
  if ( api == LINUX_OSS )
    rtapi_ = new RtApiOss();
#endif
#if defined(__WINDOWS_ASIO__)
  if ( api == WINDOWS_ASIO )
    rtapi_ = new RtApiAsio();
#endif
#if defined(__WINDOWS_WASAPI__)
  if ( api == WINDOWS_WASAPI )
    rtapi_ = new RtApiWasapi();
#endif
#if defined(__WINDOWS_DS__)
  if ( api == WINDOWS_DS )
    rtapi_ = new RtApiDs();
#endif
#if defined(__MACOSX_CORE__)
  if ( api == MACOSX_CORE )
    rtapi_ = new RtApiCore();
#endif
#if defined(__RTAUDIO_DUMMY__)
  if ( api == RTAUDIO_DUMMY )
    rtapi_ = new RtApiDummy();
#endif
}

RtAudio :: RtAudio( RtAudio::Api api )
{
  rtapi_ = 0;

  if ( api != UNSPECIFIED ) {
    // Attempt to open the specified API.
    openRtApi( api );
    if ( rtapi_ ) return;

    // No compiled support for specified API value.  Issue a debug
    // warning and continue as if no API was specified.
    UE_LOGF(LogAudioCaptureCore, Warning, "RtAudio has no compiled support for specified API argument.");
  }

  // Iterate through the compiled APIs and return as soon as we find
  // one with at least one device or we reach the end of the list.
  std::vector< RtAudio::Api > apis;
  getCompiledApi( apis );
  for ( unsigned int i=0; i<apis.size(); i++ ) {
    openRtApi( apis[i] );
    if ( rtapi_ && rtapi_->getDeviceCount() ) break;
  }

  if ( rtapi_ ) return;

  // It should not be possible to get here because the preprocessor
  // definition __RTAUDIO_DUMMY__ is automatically defined if no
  // API-specific definitions are passed to the compiler. But just in
  // case something weird happens, we'll log an error.
  UE_LOGF(LogAudioCaptureCore, Error, "RtAudio has no compiled API support found.");
}

RtAudio :: ~RtAudio()
{
  if ( rtapi_ )
    delete rtapi_;
}

void RtAudio :: openStream( RtAudio::StreamParameters *outputParameters,
                            RtAudio::StreamParameters *inputParameters,
                            RtAudioFormat format, unsigned int sampleRate,
                            unsigned int *bufferFrames,
                            RtAudioCallback callback, void *userData,
                            RtAudio::StreamOptions *options,
                            RtAudioErrorCallback errorCallback )
{
  return rtapi_->openStream( outputParameters, inputParameters, format,
                             sampleRate, bufferFrames, callback,
                             userData, options, errorCallback );
}

// *************************************************** //
//
// Public RtApi definitions (see end of file for
// private or protected utility functions).
//
// *************************************************** //

RtApi :: RtApi()
{
  stream_.state = STREAM_CLOSED;
  stream_.mode = UNINITIALIZED;
  stream_.apiHandle = 0;
  stream_.userBuffer[0] = 0;
  stream_.userBuffer[1] = 0;
  MUTEX_INITIALIZE( &stream_.mutex );
  showWarnings_ = true;
  firstErrorOccurred_ = false;
}

RtApi :: ~RtApi()
{
  MUTEX_DESTROY( &stream_.mutex );
}

void RtApi :: openStream( RtAudio::StreamParameters *oParams,
                          RtAudio::StreamParameters *iParams,
                          RtAudioFormat format, unsigned int sampleRate,
                          unsigned int *bufferFrames,
                          RtAudioCallback callback, void *userData,
                          RtAudio::StreamOptions *options,
                          RtAudioErrorCallback errorCallback )
{
  if ( stream_.state != STREAM_CLOSED ) {
    errorText_ = "RtApi::openStream: a stream is already open!";
    error( RtAudioError::INVALID_USE );
    return;
  }

  // Clear stream information potentially left from a previously open stream.
  clearStreamInfo();

  if ( oParams && oParams->nChannels < 1 ) {
    errorText_ = "RtApi::openStream: a non-NULL output StreamParameters structure cannot have an nChannels value less than one.";
    error( RtAudioError::INVALID_USE );
    return;
  }

  if ( iParams && iParams->nChannels < 1 ) {
    errorText_ = "RtApi::openStream: a non-NULL input StreamParameters structure cannot have an nChannels value less than one.";
    error( RtAudioError::INVALID_USE );
    return;
  }

  if ( oParams == NULL && iParams == NULL ) {
    errorText_ = "RtApi::openStream: input and output StreamParameters structures are both NULL!";
    error( RtAudioError::INVALID_USE );
    return;
  }

  if ( formatBytes(format) == 0 ) {
    errorText_ = "RtApi::openStream: 'format' parameter value is undefined.";
    error( RtAudioError::INVALID_USE );
    return;
  }

  unsigned int nDevices = getDeviceCount();
  unsigned int oChannels = 0;
  if ( oParams ) {
    oChannels = oParams->nChannels;
    if ( oParams->deviceId >= nDevices ) {
      errorText_ = "RtApi::openStream: output device parameter value is invalid.";
      error( RtAudioError::INVALID_USE );
      return;
    }
  }

  unsigned int iChannels = 0;
  if ( iParams ) {
    iChannels = iParams->nChannels;
    if ( iParams->deviceId >= nDevices ) {
      errorText_ = "RtApi::openStream: input device parameter value is invalid.";
      error( RtAudioError::INVALID_USE );
      return;
    }
  }

  bool result;

  if ( oChannels > 0 ) {

    result = probeDeviceOpen( oParams->deviceId, OUTPUT, oChannels, oParams->firstChannel,
                              sampleRate, format, bufferFrames, options );
    if ( result == false ) {
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }
  }

  if ( iChannels > 0 ) {

    result = probeDeviceOpen( iParams->deviceId, INPUT, iChannels, iParams->firstChannel,
                              sampleRate, format, bufferFrames, options );
    if ( result == false ) {
      if ( oChannels > 0 ) closeStream();
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }
  }

  stream_.callbackInfo.callback = (void *) callback;
  stream_.callbackInfo.userData = userData;
  stream_.callbackInfo.errorCallback = (void *) errorCallback;

  if ( options ) options->numberOfBuffers = stream_.nBuffers;
  stream_.state = STREAM_STOPPED;
}

unsigned int RtApi :: getDefaultInputDevice( void )
{
  // Should be implemented in subclasses if possible.
  return 0;
}

unsigned int RtApi :: getDefaultOutputDevice( void )
{
  // Should be implemented in subclasses if possible.
  return 0;
}

void RtApi :: closeStream( void )
{
  // MUST be implemented in subclasses!
  return;
}

bool RtApi :: probeDeviceOpen( unsigned int /*device*/, StreamMode /*mode*/, unsigned int /*channels*/,
                               unsigned int /*firstChannel*/, unsigned int /*sampleRate*/,
                               RtAudioFormat /*format*/, unsigned int * /*bufferSize*/,
                               RtAudio::StreamOptions * /*options*/ )
{
  // MUST be implemented in subclasses!
  return FAILURE;
}

void RtApi :: tickStreamTime( void )
{
  // Subclasses that do not provide their own implementation of
  // getStreamTime should call this function once per buffer I/O to
  // provide basic stream time support.

  stream_.streamTime += ( stream_.bufferSize * 1.0 / stream_.sampleRate );

#if defined( HAVE_GETTIMEOFDAY )
  gettimeofday( &stream_.lastTickTimestamp, NULL );
#endif
}

long RtApi :: getStreamLatency( void )
{
  verifyStream();

  long totalLatency = 0;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX )
    totalLatency = stream_.latency[0];
  if ( stream_.mode == INPUT || stream_.mode == DUPLEX )
    totalLatency += stream_.latency[1];

  return totalLatency;
}

double RtApi :: getStreamTime( void )
{
  verifyStream();

#if defined( HAVE_GETTIMEOFDAY )
  // Return a very accurate estimate of the stream time by
  // adding in the elapsed time since the last tick.
  struct timeval then;
  struct timeval now;

  if ( stream_.state != STREAM_RUNNING || stream_.streamTime == 0.0 )
    return stream_.streamTime;

  gettimeofday( &now, NULL );
  then = stream_.lastTickTimestamp;
  return stream_.streamTime +
    ((now.tv_sec + 0.000001 * now.tv_usec) -
     (then.tv_sec + 0.000001 * then.tv_usec));     
#else
  return stream_.streamTime;
#endif
}

void RtApi :: setStreamTime( double time )
{
  verifyStream();

  if ( time >= 0.0 )
    stream_.streamTime = time;
#if defined( HAVE_GETTIMEOFDAY )
  gettimeofday( &stream_.lastTickTimestamp, NULL );
#endif
}

unsigned int RtApi :: getStreamSampleRate( void )
{
 verifyStream();

 return stream_.sampleRate;
}


// *************************************************** //
//
// OS/API-specific methods.
//
// *************************************************** //

#if defined(__MACOSX_CORE__)

// The OS X CoreAudio API is designed to use a separate callback
// procedure for each of its audio devices.  A single RtAudio duplex
// stream using two different devices is supported here, though it
// cannot be guaranteed to always behave correctly because we cannot
// synchronize these two callbacks.
//
// A property listener is installed for over/underrun information.
// However, no functionality is currently provided to allow property
// listeners to trigger user handlers because it is unclear what could
// be done if a critical stream parameter (buffer size, sample rate,
// device disconnect) notification arrived.  The listeners entail
// quite a bit of extra code and most likely, a user program wouldn't
// be prepared for the result anyway.  However, we do provide a flag
// to the client callback function to inform of an over/underrun.

// A structure to hold various information related to the CoreAudio API
// implementation.
struct CoreHandle {
  AudioDeviceID id[2];    // device ids
#if defined( MAC_OS_X_VERSION_10_5 ) && ( MAC_OS_X_VERSION_MIN_REQUIRED >= MAC_OS_X_VERSION_10_5 )
  AudioDeviceIOProcID procId[2];
#endif
  UInt32 iStream[2];      // device stream index (or first if using multiple)
  UInt32 nStreams[2];     // number of streams to use
  bool xrun[2];
  char *deviceBuffer;
  pthread_cond_t condition;
  int drainCounter;       // Tracks callback counts when draining
  bool internalDrain;     // Indicates if stop is initiated from callback or not.

  CoreHandle()
    :deviceBuffer(0), drainCounter(0), internalDrain(false) { nStreams[0] = 1; nStreams[1] = 1; id[0] = 0; id[1] = 0; xrun[0] = false; xrun[1] = false; }
};

RtApiCore:: RtApiCore()
{
#if defined( AVAILABLE_MAC_OS_X_VERSION_10_6_AND_LATER )
  // This is a largely undocumented but absolutely necessary
  // requirement starting with OS-X 10.6.  If not called, queries and
  // updates to various audio device properties are not handled
  // correctly.
  CFRunLoopRef theRunLoop = NULL;
  AudioObjectPropertyAddress property = { kAudioHardwarePropertyRunLoop,
                                          kAudioObjectPropertyScopeGlobal,
										  kAudioObjectPropertyElementMain };
  OSStatus result = AudioObjectSetPropertyData( kAudioObjectSystemObject, &property, 0, NULL, sizeof(CFRunLoopRef), &theRunLoop);
  if ( result != noErr ) {
    errorText_ = "RtApiCore::RtApiCore: error setting run loop property!";
    error( RtAudioError::WARNING );
  }
#endif
}

RtApiCore :: ~RtApiCore()
{
  // The subclass destructor gets called before the base class
  // destructor, so close an existing stream before deallocating
  // apiDeviceId memory.
  if ( stream_.state != STREAM_CLOSED ) closeStream();
}

unsigned int RtApiCore :: getDeviceCount( void )
{
  // Find out how many audio devices there are, if any.
  UInt32 dataSize;
  AudioObjectPropertyAddress propertyAddress = { kAudioHardwarePropertyDevices, kAudioObjectPropertyScopeGlobal, kAudioObjectPropertyElementMain };
  OSStatus result = AudioObjectGetPropertyDataSize( kAudioObjectSystemObject, &propertyAddress, 0, NULL, &dataSize );
  if ( result != noErr ) {
    errorText_ = "RtApiCore::getDeviceCount: OS-X error getting device info!";
    error( RtAudioError::WARNING );
    return 0;
  }

  return dataSize / sizeof( AudioDeviceID );
}

unsigned int RtApiCore :: getDefaultInputDevice( void )
{
  unsigned int nDevices = getDeviceCount();
  if ( nDevices <= 1 ) return 0;

  AudioDeviceID id;
  UInt32 dataSize = sizeof( AudioDeviceID );
  AudioObjectPropertyAddress property = { kAudioHardwarePropertyDefaultInputDevice, kAudioObjectPropertyScopeGlobal, kAudioObjectPropertyElementMain };
  OSStatus result = AudioObjectGetPropertyData( kAudioObjectSystemObject, &property, 0, NULL, &dataSize, &id );
  if ( result != noErr ) {
    errorText_ = "RtApiCore::getDefaultInputDevice: OS-X system error getting device.";
    error( RtAudioError::WARNING );
    return 0;
  }

  dataSize *= nDevices;
  AudioDeviceID deviceList[ nDevices ];
  property.mSelector = kAudioHardwarePropertyDevices;
  result = AudioObjectGetPropertyData( kAudioObjectSystemObject, &property, 0, NULL, &dataSize, (void *) &deviceList );
  if ( result != noErr ) {
    errorText_ = "RtApiCore::getDefaultInputDevice: OS-X system error getting device IDs.";
    error( RtAudioError::WARNING );
    return 0;
  }

  for ( unsigned int i=0; i<nDevices; i++ )
    if ( id == deviceList[i] ) return i;

  errorText_ = "RtApiCore::getDefaultInputDevice: No default device found!";
  error( RtAudioError::WARNING );
  return 0;
}

unsigned int RtApiCore :: getDefaultOutputDevice( void )
{
  unsigned int nDevices = getDeviceCount();
  if ( nDevices <= 1 ) return 0;

  AudioDeviceID id;
  UInt32 dataSize = sizeof( AudioDeviceID );
  AudioObjectPropertyAddress property = { kAudioHardwarePropertyDefaultOutputDevice, kAudioObjectPropertyScopeGlobal, kAudioObjectPropertyElementMain };
  OSStatus result = AudioObjectGetPropertyData( kAudioObjectSystemObject, &property, 0, NULL, &dataSize, &id );
  if ( result != noErr ) {
    errorText_ = "RtApiCore::getDefaultOutputDevice: OS-X system error getting device.";
    error( RtAudioError::WARNING );
    return 0;
  }

  dataSize = sizeof( AudioDeviceID ) * nDevices;
  AudioDeviceID deviceList[ nDevices ];
  property.mSelector = kAudioHardwarePropertyDevices;
  result = AudioObjectGetPropertyData( kAudioObjectSystemObject, &property, 0, NULL, &dataSize, (void *) &deviceList );
  if ( result != noErr ) {
    errorText_ = "RtApiCore::getDefaultOutputDevice: OS-X system error getting device IDs.";
    error( RtAudioError::WARNING );
    return 0;
  }

  for ( unsigned int i=0; i<nDevices; i++ )
    if ( id == deviceList[i] ) return i;

  errorText_ = "RtApiCore::getDefaultOutputDevice: No default device found!";
  error( RtAudioError::WARNING );
  return 0;
}

RtAudio::DeviceInfo RtApiCore :: getDeviceInfo( unsigned int device )
{
  RtAudio::DeviceInfo info;
  info.probed = false;

  // Get device ID
  unsigned int nDevices = getDeviceCount();
  if ( nDevices == 0 ) {
    errorText_ = "RtApiCore::getDeviceInfo: no devices found!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  if ( device >= nDevices ) {
    errorText_ = "RtApiCore::getDeviceInfo: device ID is invalid!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  AudioDeviceID deviceList[ nDevices ];
  UInt32 dataSize = sizeof( AudioDeviceID ) * nDevices;
  AudioObjectPropertyAddress property = { kAudioHardwarePropertyDevices,
                                          kAudioObjectPropertyScopeGlobal,
										  kAudioObjectPropertyElementMain };
  OSStatus result = AudioObjectGetPropertyData( kAudioObjectSystemObject, &property,
                                                0, NULL, &dataSize, (void *) &deviceList );
  if ( result != noErr ) {
    errorText_ = "RtApiCore::getDeviceInfo: OS-X system error getting device IDs.";
    error( RtAudioError::WARNING );
    return info;
  }

  AudioDeviceID id = deviceList[ device ];

  // Get the device name.
  info.name.erase();
  CFStringRef cfname;
  dataSize = sizeof( CFStringRef );
  property.mSelector = kAudioObjectPropertyManufacturer;
  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &cfname );
  if ( result != noErr ) {
    errorStream_ << "RtApiCore::probeDeviceInfo: system error (" << getErrorCode( result ) << ") getting device manufacturer.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  property.mSelector = kAudioObjectPropertyName;
  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &cfname );
  if ( result != noErr ) {
    errorStream_ << "RtApiCore::probeDeviceInfo: system error (" << getErrorCode( result ) << ") getting device name.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  //const char *name = CFStringGetCStringPtr( cfname, CFStringGetSystemEncoding() );
  int length = CFStringGetLength(cfname);
  char *name = (char *)malloc(length * 3 + 1);
#if defined( UNICODE ) || defined( _UNICODE )
  CFStringGetCString(cfname, name, length * 3 + 1, kCFStringEncodingUTF8);
#else
  CFStringGetCString(cfname, name, length * 3 + 1, CFStringGetSystemEncoding());
#endif
  info.name.append( (const char *)name, strlen(name) );
  CFRelease( cfname );
  free(name);

  // Get the output stream "configuration".
  AudioBufferList	*bufferList = nil;
  property.mSelector = kAudioDevicePropertyStreamConfiguration;
  property.mScope = kAudioDevicePropertyScopeOutput;
  //  property.mElement = kAudioObjectPropertyElementWildcard;
  dataSize = 0;
  result = AudioObjectGetPropertyDataSize( id, &property, 0, NULL, &dataSize );
  if ( result != noErr || dataSize == 0 ) {
    errorStream_ << "RtApiCore::getDeviceInfo: system error (" << getErrorCode( result ) << ") getting output stream configuration info for device (" << device << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Allocate the AudioBufferList.
  bufferList = (AudioBufferList *) malloc( dataSize );
  if ( bufferList == NULL ) {
    errorText_ = "RtApiCore::getDeviceInfo: memory error allocating output AudioBufferList.";
    error( RtAudioError::WARNING );
    return info;
  }

  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, bufferList );
  if ( result != noErr || dataSize == 0 ) {
    free( bufferList );
    errorStream_ << "RtApiCore::getDeviceInfo: system error (" << getErrorCode( result ) << ") getting output stream configuration for device (" << device << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Get output channel information.
  unsigned int i, nStreams = bufferList->mNumberBuffers;
  for ( i=0; i<nStreams; i++ )
    info.outputChannels += bufferList->mBuffers[i].mNumberChannels;
  free( bufferList );

  // Get the input stream "configuration".
  property.mScope = kAudioDevicePropertyScopeInput;
  result = AudioObjectGetPropertyDataSize( id, &property, 0, NULL, &dataSize );
  if ( result != noErr || dataSize == 0 ) {
    errorStream_ << "RtApiCore::getDeviceInfo: system error (" << getErrorCode( result ) << ") getting input stream configuration info for device (" << device << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Allocate the AudioBufferList.
  bufferList = (AudioBufferList *) malloc( dataSize );
  if ( bufferList == NULL ) {
    errorText_ = "RtApiCore::getDeviceInfo: memory error allocating input AudioBufferList.";
    error( RtAudioError::WARNING );
    return info;
  }

  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, bufferList );
  if (result != noErr || dataSize == 0) {
    free( bufferList );
    errorStream_ << "RtApiCore::getDeviceInfo: system error (" << getErrorCode( result ) << ") getting input stream configuration for device (" << device << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Get input channel information.
  nStreams = bufferList->mNumberBuffers;
  for ( i=0; i<nStreams; i++ )
    info.inputChannels += bufferList->mBuffers[i].mNumberChannels;
  free( bufferList );

  // If device opens for both playback and capture, we determine the channels.
  if ( info.outputChannels > 0 && info.inputChannels > 0 )
    info.duplexChannels = (info.outputChannels > info.inputChannels) ? info.inputChannels : info.outputChannels;

  // Probe the device sample rates.
  bool isInput = false;
  if ( info.outputChannels == 0 ) isInput = true;

  // Determine the supported sample rates.
  property.mSelector = kAudioDevicePropertyAvailableNominalSampleRates;
  if ( isInput == false ) property.mScope = kAudioDevicePropertyScopeOutput;
  result = AudioObjectGetPropertyDataSize( id, &property, 0, NULL, &dataSize );
  if ( result != kAudioHardwareNoError || dataSize == 0 ) {
    errorStream_ << "RtApiCore::getDeviceInfo: system error (" << getErrorCode( result ) << ") getting sample rate info.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  UInt32 nRanges = dataSize / sizeof( AudioValueRange );
  AudioValueRange rangeList[ nRanges ];
  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &rangeList );
  if ( result != kAudioHardwareNoError ) {
    errorStream_ << "RtApiCore::getDeviceInfo: system error (" << getErrorCode( result ) << ") getting sample rates.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // The sample rate reporting mechanism is a bit of a mystery.  It
  // seems that it can either return individual rates or a range of
  // rates.  I assume that if the min / max range values are the same,
  // then that represents a single supported rate and if the min / max
  // range values are different, the device supports an arbitrary
  // range of values (though there might be multiple ranges, so we'll
  // use the most conservative range).
  Float64 minimumRate = 1.0, maximumRate = 10000000000.0;
  bool haveValueRange = false;
  info.sampleRates.clear();
  for ( UInt32 i2=0; i2 <nRanges; i2++ ) {
    if ( rangeList[i2].mMinimum == rangeList[i2].mMaximum ) {
      unsigned int tmpSr = (unsigned int) rangeList[i2].mMinimum;
      info.sampleRates.push_back( tmpSr );

      if ( !info.preferredSampleRate || ( tmpSr <= 48000 && tmpSr > info.preferredSampleRate ) )
        info.preferredSampleRate = tmpSr;

    } else {
      haveValueRange = true;
      if ( rangeList[i2].mMinimum > minimumRate ) minimumRate = rangeList[i2].mMinimum;
      if ( rangeList[i2].mMaximum < maximumRate ) maximumRate = rangeList[i2].mMaximum;
    }
  }

  if ( haveValueRange ) {
    for ( unsigned int k=0; k<MAX_SAMPLE_RATES; k++ ) {
      if ( SAMPLE_RATES[k] >= (unsigned int) minimumRate && SAMPLE_RATES[k] <= (unsigned int) maximumRate ) {
        info.sampleRates.push_back( SAMPLE_RATES[k] );

        if ( !info.preferredSampleRate || ( SAMPLE_RATES[k] <= 48000 && SAMPLE_RATES[k] > info.preferredSampleRate ) )
          info.preferredSampleRate = SAMPLE_RATES[k];
      }
    }
  }

  // Sort and remove any redundant values
  std::sort( info.sampleRates.begin(), info.sampleRates.end() );
  info.sampleRates.erase( unique( info.sampleRates.begin(), info.sampleRates.end() ), info.sampleRates.end() );

  if ( info.sampleRates.size() == 0 ) {
    errorStream_ << "RtApiCore::probeDeviceInfo: No supported sample rates found for device (" << device << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // CoreAudio always uses 32-bit floating point data for PCM streams.
  // Thus, any other "physical" formats supported by the device are of
  // no interest to the client.
  info.nativeFormats = RTAUDIO_FLOAT32;

  if ( info.outputChannels > 0 )
    if ( getDefaultOutputDevice() == device ) info.isDefaultOutput = true;
  if ( info.inputChannels > 0 )
    if ( getDefaultInputDevice() == device ) info.isDefaultInput = true;

  info.probed = true;
  return info;
}

static OSStatus callbackHandler( AudioDeviceID inDevice,
                                 const AudioTimeStamp* /*inNow*/,
                                 const AudioBufferList* inInputData,
                                 const AudioTimeStamp* /*inInputTime*/,
                                 AudioBufferList* outOutputData,
                                 const AudioTimeStamp* /*inOutputTime*/,
                                 void* infoPointer )
{
  CallbackInfo *info = (CallbackInfo *) infoPointer;

  RtApiCore *object = (RtApiCore *) info->object;
  if ( object->callbackEvent( inDevice, inInputData, outOutputData ) == false )
    return kAudioHardwareUnspecifiedError;
  else
    return kAudioHardwareNoError;
}

static OSStatus xrunListener( AudioObjectID /*inDevice*/,
                              UInt32 nAddresses,
                              const AudioObjectPropertyAddress properties[],
                              void* handlePointer )
{
  CoreHandle *handle = (CoreHandle *) handlePointer;
  for ( UInt32 i=0; i<nAddresses; i++ ) {
    if ( properties[i].mSelector == kAudioDeviceProcessorOverload ) {
      if ( properties[i].mScope == kAudioDevicePropertyScopeInput )
        handle->xrun[1] = true;
      else
        handle->xrun[0] = true;
    }
  }

  return kAudioHardwareNoError;
}

static OSStatus rateListener( AudioObjectID inDevice,
                              UInt32 /*nAddresses*/,
                              const AudioObjectPropertyAddress /*properties*/[],
                              void* ratePointer )
{
  Float64 *rate = (Float64 *) ratePointer;
  UInt32 dataSize = sizeof( Float64 );
  AudioObjectPropertyAddress property = { kAudioDevicePropertyNominalSampleRate,
                                          kAudioObjectPropertyScopeGlobal,
										  kAudioObjectPropertyElementMain };
  AudioObjectGetPropertyData( inDevice, &property, 0, NULL, &dataSize, rate );
  return kAudioHardwareNoError;
}

bool RtApiCore :: probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                                   unsigned int firstChannel, unsigned int sampleRate,
                                   RtAudioFormat format, unsigned int *bufferSize,
                                   RtAudio::StreamOptions *options )
{
  // Get device ID
  unsigned int nDevices = getDeviceCount();
  if ( nDevices == 0 ) {
    // This should not happen because a check is made before this function is called.
    errorText_ = "RtApiCore::probeDeviceOpen: no devices found!";
    return FAILURE;
  }

  if ( device >= nDevices ) {
    // This should not happen because a check is made before this function is called.
    errorText_ = "RtApiCore::probeDeviceOpen: device ID is invalid!";
    return FAILURE;
  }

  AudioDeviceID deviceList[ nDevices ];
  UInt32 dataSize = sizeof( AudioDeviceID ) * nDevices;
  AudioObjectPropertyAddress property = { kAudioHardwarePropertyDevices,
                                          kAudioObjectPropertyScopeGlobal,
										  kAudioObjectPropertyElementMain };
  OSStatus result = AudioObjectGetPropertyData( kAudioObjectSystemObject, &property,
                                                0, NULL, &dataSize, (void *) &deviceList );
  if ( result != noErr ) {
    errorText_ = "RtApiCore::probeDeviceOpen: OS-X system error getting device IDs.";
    return FAILURE;
  }

  AudioDeviceID id = deviceList[ device ];

  // Setup for stream mode.
  bool isInput = false;
  if ( mode == INPUT ) {
    isInput = true;
    property.mScope = kAudioDevicePropertyScopeInput;
  }
  else
    property.mScope = kAudioDevicePropertyScopeOutput;

  // Get the stream "configuration".
  AudioBufferList	*bufferList = nil;
  dataSize = 0;
  property.mSelector = kAudioDevicePropertyStreamConfiguration;
  result = AudioObjectGetPropertyDataSize( id, &property, 0, NULL, &dataSize );
  if ( result != noErr || dataSize == 0 ) {
    errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting stream configuration info for device (" << device << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Allocate the AudioBufferList.
  bufferList = (AudioBufferList *) malloc( dataSize );
  if ( bufferList == NULL ) {
    errorText_ = "RtApiCore::probeDeviceOpen: memory error allocating AudioBufferList.";
    return FAILURE;
  }

  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, bufferList );
  if (result != noErr || dataSize == 0) {
    free( bufferList );
    errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting stream configuration for device (" << device << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Search for one or more streams that contain the desired number of
  // channels. CoreAudio devices can have an arbitrary number of
  // streams and each stream can have an arbitrary number of channels.
  // For each stream, a single buffer of interleaved samples is
  // provided.  RtAudio prefers the use of one stream of interleaved
  // data or multiple consecutive single-channel streams.  However, we
  // now support multiple consecutive multi-channel streams of
  // interleaved data as well.
  UInt32 iStream, offsetCounter = firstChannel;
  UInt32 nStreams = bufferList->mNumberBuffers;
  bool monoMode = false;
  bool foundStream = false;

  // First check that the device supports the requested number of
  // channels.
  UInt32 deviceChannels = 0;
  for ( iStream=0; iStream<nStreams; iStream++ )
    deviceChannels += bufferList->mBuffers[iStream].mNumberChannels;

  if ( deviceChannels < ( channels + firstChannel ) ) {
    free( bufferList );
    errorStream_ << "RtApiCore::probeDeviceOpen: the device (" << device << ") does not support the requested channel count.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Look for a single stream meeting our needs.
  UInt32 firstStream, streamCount = 1, streamChannels = 0, channelOffset = 0;
  for ( iStream=0; iStream<nStreams; iStream++ ) {
    streamChannels = bufferList->mBuffers[iStream].mNumberChannels;
    if ( streamChannels >= channels + offsetCounter ) {
      firstStream = iStream;
      channelOffset = offsetCounter;
      foundStream = true;
      break;
    }
    if ( streamChannels > offsetCounter ) break;
    offsetCounter -= streamChannels;
  }

  // If we didn't find a single stream above, then we should be able
  // to meet the channel specification with multiple streams.
  if ( foundStream == false ) {
    monoMode = true;
    offsetCounter = firstChannel;
    for ( iStream=0; iStream<nStreams; iStream++ ) {
      streamChannels = bufferList->mBuffers[iStream].mNumberChannels;
      if ( streamChannels > offsetCounter ) break;
      offsetCounter -= streamChannels;
    }

    firstStream = iStream;
    channelOffset = offsetCounter;
    Int32 channelCounter = channels + offsetCounter - streamChannels;

    if ( streamChannels > 1 ) monoMode = false;
    while ( channelCounter > 0 ) {
      streamChannels = bufferList->mBuffers[++iStream].mNumberChannels;
      if ( streamChannels > 1 ) monoMode = false;
      channelCounter -= streamChannels;
      streamCount++;
    }
  }

  free( bufferList );

  // Determine the buffer size.
  AudioValueRange	bufferRange;
  dataSize = sizeof( AudioValueRange );
  property.mSelector = kAudioDevicePropertyBufferFrameSizeRange;
  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &bufferRange );

  if ( result != noErr ) {
    errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting buffer size range for device (" << device << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  if ( bufferRange.mMinimum > *bufferSize ) *bufferSize = (unsigned long) bufferRange.mMinimum;
  else if ( bufferRange.mMaximum < *bufferSize ) *bufferSize = (unsigned long) bufferRange.mMaximum;
  if ( options && options->flags & RTAUDIO_MINIMIZE_LATENCY ) *bufferSize = (unsigned long) bufferRange.mMinimum;

  // Set the buffer size.  For multiple streams, I'm assuming we only
  // need to make this setting for the master channel.
  UInt32 theSize = (UInt32) *bufferSize;
  dataSize = sizeof( UInt32 );
  property.mSelector = kAudioDevicePropertyBufferFrameSize;
  result = AudioObjectSetPropertyData( id, &property, 0, NULL, dataSize, &theSize );

  if ( result != noErr ) {
    errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") setting the buffer size for device (" << device << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // If attempting to setup a duplex stream, the bufferSize parameter
  // MUST be the same in both directions!
  *bufferSize = theSize;
  if ( stream_.mode == OUTPUT && mode == INPUT && *bufferSize != stream_.bufferSize ) {
    errorStream_ << "RtApiCore::probeDeviceOpen: system error setting buffer size for duplex stream on device (" << device << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  stream_.bufferSize = *bufferSize;
  stream_.nBuffers = 1;

  // Try to set "hog" mode ... it's not clear to me this is working.
  if ( options && options->flags & RTAUDIO_HOG_DEVICE ) {
    pid_t hog_pid;
    dataSize = sizeof( hog_pid );
    property.mSelector = kAudioDevicePropertyHogMode;
    result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &hog_pid );
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting 'hog' state!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    if ( hog_pid != getpid() ) {
      hog_pid = getpid();
      result = AudioObjectSetPropertyData( id, &property, 0, NULL, dataSize, &hog_pid );
      if ( result != noErr ) {
        errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") setting 'hog' state!";
        errorText_ = errorStream_.str();
        return FAILURE;
      }
    }
  }

  // Check and if necessary, change the sample rate for the device.
  Float64 nominalRate;
  dataSize = sizeof( Float64 );
  property.mSelector = kAudioDevicePropertyNominalSampleRate;
  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &nominalRate );
  if ( result != noErr ) {
    errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting current sample rate.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Only change the sample rate if off by more than 1 Hz.
  if ( fabs( nominalRate - (double)sampleRate ) > 1.0 ) {

    // Set a property listener for the sample rate change
    Float64 reportedRate = 0.0;
    AudioObjectPropertyAddress tmp = { kAudioDevicePropertyNominalSampleRate, kAudioObjectPropertyScopeGlobal, kAudioObjectPropertyElementMain };
    result = AudioObjectAddPropertyListener( id, &tmp, rateListener, (void *) &reportedRate );
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") setting sample rate property listener for device (" << device << ").";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    nominalRate = (Float64) sampleRate;
    result = AudioObjectSetPropertyData( id, &property, 0, NULL, dataSize, &nominalRate );
    if ( result != noErr ) {
      AudioObjectRemovePropertyListener( id, &tmp, rateListener, (void *) &reportedRate );
      errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") setting sample rate for device (" << device << ").";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Now wait until the reported nominal rate is what we just set.
    UInt32 microCounter = 0;
    while ( reportedRate != nominalRate ) {
      microCounter += 5000;
      if ( microCounter > 5000000 ) break;
      usleep( 5000 );
    }

    // Remove the property listener.
    AudioObjectRemovePropertyListener( id, &tmp, rateListener, (void *) &reportedRate );

    if ( microCounter > 5000000 ) {
      errorStream_ << "RtApiCore::probeDeviceOpen: timeout waiting for sample rate update for device (" << device << ").";
      errorText_ = errorStream_.str();
      return FAILURE;
    }
  }

  // Now set the stream format for all streams.  Also, check the
  // physical format of the device and change that if necessary.
  AudioStreamBasicDescription	description;
  dataSize = sizeof( AudioStreamBasicDescription );
  property.mSelector = kAudioStreamPropertyVirtualFormat;
  result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &description );
  if ( result != noErr ) {
    errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting stream format for device (" << device << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Set the sample rate and data format id.  However, only make the
  // change if the sample rate is not within 1.0 of the desired
  // rate and the format is not linear pcm.
  bool updateFormat = false;
  if ( fabs( description.mSampleRate - (Float64)sampleRate ) > 1.0 ) {
    description.mSampleRate = (Float64) sampleRate;
    updateFormat = true;
  }

  if ( description.mFormatID != kAudioFormatLinearPCM ) {
    description.mFormatID = kAudioFormatLinearPCM;
    updateFormat = true;
  }

  if ( updateFormat ) {
    result = AudioObjectSetPropertyData( id, &property, 0, NULL, dataSize, &description );
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") setting sample rate or data format for device (" << device << ").";
      errorText_ = errorStream_.str();
      return FAILURE;
    }
  }

  // Now check the physical format.
  property.mSelector = kAudioStreamPropertyPhysicalFormat;
  result = AudioObjectGetPropertyData( id, &property, 0, NULL,  &dataSize, &description );
  if ( result != noErr ) {
    errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting stream physical format for device (" << device << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  //std::cout << "Current physical stream format:" << std::endl;
  //std::cout << "   mBitsPerChan = " << description.mBitsPerChannel << std::endl;
  //std::cout << "   aligned high = " << (description.mFormatFlags & kAudioFormatFlagIsAlignedHigh) << ", isPacked = " << (description.mFormatFlags & kAudioFormatFlagIsPacked) << std::endl;
  //std::cout << "   bytesPerFrame = " << description.mBytesPerFrame << std::endl;
  //std::cout << "   sample rate = " << description.mSampleRate << std::endl;

  if ( description.mFormatID != kAudioFormatLinearPCM || description.mBitsPerChannel < 16 ) {
    description.mFormatID = kAudioFormatLinearPCM;
    //description.mSampleRate = (Float64) sampleRate;
    AudioStreamBasicDescription	testDescription = description;
    UInt32 formatFlags;

    // We'll try higher bit rates first and then work our way down.
    std::vector< std::pair<UInt32, UInt32>  > physicalFormats;
    formatFlags = (description.mFormatFlags | kLinearPCMFormatFlagIsFloat) & ~kLinearPCMFormatFlagIsSignedInteger;
    physicalFormats.push_back( std::pair<Float32, UInt32>( 32, formatFlags ) );
    formatFlags = (description.mFormatFlags | kLinearPCMFormatFlagIsSignedInteger | kAudioFormatFlagIsPacked) & ~kLinearPCMFormatFlagIsFloat;
    physicalFormats.push_back( std::pair<Float32, UInt32>( 32, formatFlags ) );
    physicalFormats.push_back( std::pair<Float32, UInt32>( 24, formatFlags ) );   // 24-bit packed
    formatFlags &= ~( kAudioFormatFlagIsPacked | kAudioFormatFlagIsAlignedHigh );
    physicalFormats.push_back( std::pair<Float32, UInt32>( 24.2, formatFlags ) ); // 24-bit in 4 bytes, aligned low
    formatFlags |= kAudioFormatFlagIsAlignedHigh;
    physicalFormats.push_back( std::pair<Float32, UInt32>( 24.4, formatFlags ) ); // 24-bit in 4 bytes, aligned high
    formatFlags = (description.mFormatFlags | kLinearPCMFormatFlagIsSignedInteger | kAudioFormatFlagIsPacked) & ~kLinearPCMFormatFlagIsFloat;
    physicalFormats.push_back( std::pair<Float32, UInt32>( 16, formatFlags ) );
    physicalFormats.push_back( std::pair<Float32, UInt32>( 8, formatFlags ) );

    bool setPhysicalFormat = false;
    for( unsigned int i=0; i<physicalFormats.size(); i++ ) {
      testDescription = description;
      testDescription.mBitsPerChannel = (UInt32) physicalFormats[i].first;
      testDescription.mFormatFlags = physicalFormats[i].second;
      if ( (24 == (UInt32)physicalFormats[i].first) && ~( physicalFormats[i].second & kAudioFormatFlagIsPacked ) )
        testDescription.mBytesPerFrame =  4 * testDescription.mChannelsPerFrame;
      else
        testDescription.mBytesPerFrame =  testDescription.mBitsPerChannel/8 * testDescription.mChannelsPerFrame;
      testDescription.mBytesPerPacket = testDescription.mBytesPerFrame * testDescription.mFramesPerPacket;
      result = AudioObjectSetPropertyData( id, &property, 0, NULL, dataSize, &testDescription );
      if ( result == noErr ) {
        setPhysicalFormat = true;
        //std::cout << "Updated physical stream format:" << std::endl;
        //std::cout << "   mBitsPerChan = " << testDescription.mBitsPerChannel << std::endl;
        //std::cout << "   aligned high = " << (testDescription.mFormatFlags & kAudioFormatFlagIsAlignedHigh) << ", isPacked = " << (testDescription.mFormatFlags & kAudioFormatFlagIsPacked) << std::endl;
        //std::cout << "   bytesPerFrame = " << testDescription.mBytesPerFrame << std::endl;
        //std::cout << "   sample rate = " << testDescription.mSampleRate << std::endl;
        break;
      }
    }

    if ( !setPhysicalFormat ) {
      errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") setting physical data format for device (" << device << ").";
      errorText_ = errorStream_.str();
      return FAILURE;
    }
  } // done setting virtual/physical formats.

  // Get the stream / device latency.
  UInt32 latency;
  dataSize = sizeof( UInt32 );
  property.mSelector = kAudioDevicePropertyLatency;
  if ( AudioObjectHasProperty( id, &property ) == true ) {
    result = AudioObjectGetPropertyData( id, &property, 0, NULL, &dataSize, &latency );
    if ( result == kAudioHardwareNoError ) stream_.latency[ mode ] = latency;
    else {
      errorStream_ << "RtApiCore::probeDeviceOpen: system error (" << getErrorCode( result ) << ") getting device latency for device (" << device << ").";
      errorText_ = errorStream_.str();
      error( RtAudioError::WARNING );
    }
  }

  // Byte-swapping: According to AudioHardware.h, the stream data will
  // always be presented in native-endian format, so we should never
  // need to byte swap.
  stream_.doByteSwap[mode] = false;

  // From the CoreAudio documentation, PCM data must be supplied as
  // 32-bit floats.
  stream_.userFormat = format;
  stream_.deviceFormat[mode] = RTAUDIO_FLOAT32;

  if ( streamCount == 1 )
    stream_.nDeviceChannels[mode] = description.mChannelsPerFrame;
  else // multiple streams
    stream_.nDeviceChannels[mode] = channels;
  stream_.nUserChannels[mode] = channels;
  stream_.channelOffset[mode] = channelOffset;  // offset within a CoreAudio stream
  if ( options && options->flags & RTAUDIO_NONINTERLEAVED ) stream_.userInterleaved = false;
  else stream_.userInterleaved = true;
  stream_.deviceInterleaved[mode] = true;
  if ( monoMode == true ) stream_.deviceInterleaved[mode] = false;

  // Set flags for buffer conversion.
  stream_.doConvertBuffer[mode] = false;
  if ( stream_.userFormat != stream_.deviceFormat[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.nUserChannels[mode] < stream_.nDeviceChannels[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( streamCount == 1 ) {
    if ( stream_.nUserChannels[mode] > 1 &&
         stream_.userInterleaved != stream_.deviceInterleaved[mode] )
      stream_.doConvertBuffer[mode] = true;
  }
  else if ( monoMode && stream_.userInterleaved )
    stream_.doConvertBuffer[mode] = true;

  // Allocate our CoreHandle structure for the stream.
  CoreHandle *handle = 0;
  if ( stream_.apiHandle == 0 ) {
    try {
      handle = new CoreHandle;
    }
    catch ( std::bad_alloc& ) {
      errorText_ = "RtApiCore::probeDeviceOpen: error allocating CoreHandle memory.";
      goto error;
    }

    if ( pthread_cond_init( &handle->condition, NULL ) ) {
      errorText_ = "RtApiCore::probeDeviceOpen: error initializing pthread condition variable.";
      goto error;
    }
    stream_.apiHandle = (void *) handle;
  }
  else
    handle = (CoreHandle *) stream_.apiHandle;
  handle->iStream[mode] = firstStream;
  handle->nStreams[mode] = streamCount;
  handle->id[mode] = id;

  // Allocate necessary internal buffers.
  unsigned long bufferBytes;
  bufferBytes = stream_.nUserChannels[mode] * *bufferSize * formatBytes( stream_.userFormat );
  //  stream_.userBuffer[mode] = (char *) calloc( bufferBytes, 1 );
  stream_.userBuffer[mode] = (char *) malloc( bufferBytes * sizeof(char) );
  memset( stream_.userBuffer[mode], 0, bufferBytes * sizeof(char) );
  if ( stream_.userBuffer[mode] == NULL ) {
    errorText_ = "RtApiCore::probeDeviceOpen: error allocating user buffer memory.";
    goto error;
  }

  // If possible, we will make use of the CoreAudio stream buffers as
  // "device buffers".  However, we can't do this if using multiple
  // streams.
  if ( stream_.doConvertBuffer[mode] && handle->nStreams[mode] > 1 ) {

    bool makeBuffer = true;
    bufferBytes = stream_.nDeviceChannels[mode] * formatBytes( stream_.deviceFormat[mode] );
    if ( mode == INPUT ) {
      if ( stream_.mode == OUTPUT && stream_.deviceBuffer ) {
        unsigned long bytesOut = stream_.nDeviceChannels[0] * formatBytes( stream_.deviceFormat[0] );
        if ( bufferBytes <= bytesOut ) makeBuffer = false;
      }
    }

    if ( makeBuffer ) {
      bufferBytes *= *bufferSize;
      if ( stream_.deviceBuffer ) free( stream_.deviceBuffer );
      stream_.deviceBuffer = (char *) calloc( bufferBytes, 1 );
      if ( stream_.deviceBuffer == NULL ) {
        errorText_ = "RtApiCore::probeDeviceOpen: error allocating device buffer memory.";
        goto error;
      }
    }
  }

  stream_.sampleRate = sampleRate;
  stream_.device[mode] = device;
  stream_.state = STREAM_STOPPED;
  stream_.callbackInfo.object = (void *) this;

  // Setup the buffer conversion information structure.
  if ( stream_.doConvertBuffer[mode] ) {
    if ( streamCount > 1 ) setConvertInfo( mode, 0 );
    else setConvertInfo( mode, channelOffset );
  }

  if ( mode == INPUT && stream_.mode == OUTPUT && stream_.device[0] == device )
    // Only one callback procedure per device.
    stream_.mode = DUPLEX;
  else {
#if defined( MAC_OS_X_VERSION_10_5 ) && ( MAC_OS_X_VERSION_MIN_REQUIRED >= MAC_OS_X_VERSION_10_5 )
    result = AudioDeviceCreateIOProcID( id, callbackHandler, (void *) &stream_.callbackInfo, &handle->procId[mode] );
#else
    // deprecated in favor of AudioDeviceCreateIOProcID()
    result = AudioDeviceAddIOProc( id, callbackHandler, (void *) &stream_.callbackInfo );
#endif
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::probeDeviceOpen: system error setting callback for device (" << device << ").";
      errorText_ = errorStream_.str();
      goto error;
    }
    if ( stream_.mode == OUTPUT && mode == INPUT )
      stream_.mode = DUPLEX;
    else
      stream_.mode = mode;
  }

  // Setup the device property listener for over/underload.
  property.mSelector = kAudioDeviceProcessorOverload;
  property.mScope = kAudioObjectPropertyScopeGlobal;
  result = AudioObjectAddPropertyListener( id, &property, xrunListener, (void *) handle );

  return SUCCESS;

 error:
  if ( handle ) {
    pthread_cond_destroy( &handle->condition );
    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.state = STREAM_CLOSED;
  return FAILURE;
}

void RtApiCore :: closeStream( void )
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiCore::closeStream(): no open stream to close!";
    error( RtAudioError::WARNING );
    return;
  }

  CoreHandle *handle = (CoreHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    if (handle) {
      AudioObjectPropertyAddress property = { kAudioHardwarePropertyDevices,
        kAudioObjectPropertyScopeGlobal,
		kAudioObjectPropertyElementMain };

      property.mSelector = kAudioDeviceProcessorOverload;
      property.mScope = kAudioObjectPropertyScopeGlobal;
      if (AudioObjectRemovePropertyListener( handle->id[0], &property, xrunListener, (void *) handle ) != noErr) {
        errorText_ = "RtApiCore::closeStream(): error removing property listener!";
        error( RtAudioError::WARNING );
      }
    }
    if ( stream_.state == STREAM_RUNNING )
      AudioDeviceStop( handle->id[0], callbackHandler );
#if defined( MAC_OS_X_VERSION_10_5 ) && ( MAC_OS_X_VERSION_MIN_REQUIRED >= MAC_OS_X_VERSION_10_5 )
    AudioDeviceDestroyIOProcID( handle->id[0], handle->procId[0] );
#else
    // deprecated in favor of AudioDeviceDestroyIOProcID()
    AudioDeviceRemoveIOProc( handle->id[0], callbackHandler );
#endif
  }

  if ( stream_.mode == INPUT || ( stream_.mode == DUPLEX && stream_.device[0] != stream_.device[1] ) ) {
    if (handle) {
      AudioObjectPropertyAddress property = { kAudioHardwarePropertyDevices,
        kAudioObjectPropertyScopeGlobal,
		kAudioObjectPropertyElementMain };

      property.mSelector = kAudioDeviceProcessorOverload;
      property.mScope = kAudioObjectPropertyScopeGlobal;
      if (AudioObjectRemovePropertyListener( handle->id[1], &property, xrunListener, (void *) handle ) != noErr) {
        errorText_ = "RtApiCore::closeStream(): error removing property listener!";
        error( RtAudioError::WARNING );
      }
    }
    if ( stream_.state == STREAM_RUNNING )
      AudioDeviceStop( handle->id[1], callbackHandler );
#if defined( MAC_OS_X_VERSION_10_5 ) && ( MAC_OS_X_VERSION_MIN_REQUIRED >= MAC_OS_X_VERSION_10_5 )
    AudioDeviceDestroyIOProcID( handle->id[1], handle->procId[1] );
#else
    // deprecated in favor of AudioDeviceDestroyIOProcID()
    AudioDeviceRemoveIOProc( handle->id[1], callbackHandler );
#endif
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  // Destroy pthread condition variable.
  pthread_cond_destroy( &handle->condition );
  delete handle;
  stream_.apiHandle = 0;

  stream_.mode = UNINITIALIZED;
  stream_.state = STREAM_CLOSED;
}

void RtApiCore :: startStream( void )
{
  verifyStream();
  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiCore::startStream(): the stream is already running!";
    error( RtAudioError::WARNING );
    return;
  }

  OSStatus result = noErr;
  CoreHandle *handle = (CoreHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    result = AudioDeviceStart( handle->id[0], callbackHandler );
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::startStream: system error (" << getErrorCode( result ) << ") starting callback procedure on device (" << stream_.device[0] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  if ( stream_.mode == INPUT ||
       ( stream_.mode == DUPLEX && stream_.device[0] != stream_.device[1] ) ) {

    result = AudioDeviceStart( handle->id[1], callbackHandler );
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::startStream: system error starting input callback procedure on device (" << stream_.device[1] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  handle->drainCounter = 0;
  handle->internalDrain = false;
  stream_.state = STREAM_RUNNING;

 unlock:
  if ( result == noErr ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiCore :: stopStream( void )
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiCore::stopStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  OSStatus result = noErr;
  CoreHandle *handle = (CoreHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    if ( handle->drainCounter == 0 ) {
      handle->drainCounter = 2;
      pthread_cond_wait( &handle->condition, &stream_.mutex ); // block until signaled
    }

    result = AudioDeviceStop( handle->id[0], callbackHandler );
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::stopStream: system error (" << getErrorCode( result ) << ") stopping callback procedure on device (" << stream_.device[0] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  if ( stream_.mode == INPUT || ( stream_.mode == DUPLEX && stream_.device[0] != stream_.device[1] ) ) {

    result = AudioDeviceStop( handle->id[1], callbackHandler );
    if ( result != noErr ) {
      errorStream_ << "RtApiCore::stopStream: system error (" << getErrorCode( result ) << ") stopping input callback procedure on device (" << stream_.device[1] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  stream_.state = STREAM_STOPPED;

 unlock:
  if ( result == noErr ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiCore :: abortStream( void )
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiCore::abortStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  CoreHandle *handle = (CoreHandle *) stream_.apiHandle;
  handle->drainCounter = 2;

  stopStream();
}

// This function will be called by a spawned thread when the user
// callback function signals that the stream should be stopped or
// aborted.  It is better to handle it this way because the
// callbackEvent() function probably should return before the AudioDeviceStop()
// function is called.
static void *coreStopStream( void *ptr )
{
  CallbackInfo *info = (CallbackInfo *) ptr;
  RtApiCore *object = (RtApiCore *) info->object;

  object->stopStream();
  pthread_exit( NULL );
}

bool RtApiCore :: callbackEvent( AudioDeviceID deviceId,
                                 const AudioBufferList *inBufferList,
                                 const AudioBufferList *outBufferList )
{
  if ( stream_.state == STREAM_STOPPED || stream_.state == STREAM_STOPPING ) return SUCCESS;
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiCore::callbackEvent(): the stream is closed ... this shouldn't happen!";
    error( RtAudioError::WARNING );
    return FAILURE;
  }

  CallbackInfo *info = (CallbackInfo *) &stream_.callbackInfo;
  CoreHandle *handle = (CoreHandle *) stream_.apiHandle;

  // Check if we were draining the stream and signal is finished.
  if ( handle->drainCounter > 3 ) {
    ThreadHandle threadId;

    stream_.state = STREAM_STOPPING;
    if ( handle->internalDrain == true )
      pthread_create( &threadId, NULL, coreStopStream, info );
    else // external call to stopStream()
      pthread_cond_signal( &handle->condition );
    return SUCCESS;
  }

  AudioDeviceID outputDevice = handle->id[0];

  // Invoke user callback to get fresh output data UNLESS we are
  // draining stream or duplex mode AND the input/output devices are
  // different AND this function is called for the input device.
  if ( handle->drainCounter == 0 && ( stream_.mode != DUPLEX || deviceId == outputDevice ) ) {
    RtAudioCallback callback = (RtAudioCallback) info->callback;
    double streamTime = getStreamTime();
    RtAudioStreamStatus status = 0;
    if ( stream_.mode != INPUT && handle->xrun[0] == true ) {
      status |= RTAUDIO_OUTPUT_UNDERFLOW;
      handle->xrun[0] = false;
    }
    if ( stream_.mode != OUTPUT && handle->xrun[1] == true ) {
      status |= RTAUDIO_INPUT_OVERFLOW;
      handle->xrun[1] = false;
    }

    int cbReturnValue = callback( stream_.userBuffer[0], stream_.userBuffer[1],
                                  stream_.bufferSize, streamTime, status, info->userData );
    if ( cbReturnValue == 2 ) {
      stream_.state = STREAM_STOPPING;
      handle->drainCounter = 2;
      abortStream();
      return SUCCESS;
    }
    else if ( cbReturnValue == 1 ) {
      handle->drainCounter = 1;
      handle->internalDrain = true;
    }
  }

  if ( stream_.mode == OUTPUT || ( stream_.mode == DUPLEX && deviceId == outputDevice ) ) {

    if ( handle->drainCounter > 1 ) { // write zeros to the output stream

      if ( handle->nStreams[0] == 1 ) {
        memset( outBufferList->mBuffers[handle->iStream[0]].mData,
                0,
                outBufferList->mBuffers[handle->iStream[0]].mDataByteSize );
      }
      else { // fill multiple streams with zeros
        for ( unsigned int i=0; i<handle->nStreams[0]; i++ ) {
          memset( outBufferList->mBuffers[handle->iStream[0]+i].mData,
                  0,
                  outBufferList->mBuffers[handle->iStream[0]+i].mDataByteSize );
        }
      }
    }
    else if ( handle->nStreams[0] == 1 ) {
      if ( stream_.doConvertBuffer[0] ) { // convert directly to CoreAudio stream buffer
        convertBuffer( (char *) outBufferList->mBuffers[handle->iStream[0]].mData,
                       stream_.userBuffer[0], stream_.convertInfo[0] );
      }
      else { // copy from user buffer
        memcpy( outBufferList->mBuffers[handle->iStream[0]].mData,
                stream_.userBuffer[0],
                outBufferList->mBuffers[handle->iStream[0]].mDataByteSize );
      }
    }
    else { // fill multiple streams
      Float32 *inBuffer = (Float32 *) stream_.userBuffer[0];
      if ( stream_.doConvertBuffer[0] ) {
        convertBuffer( stream_.deviceBuffer, stream_.userBuffer[0], stream_.convertInfo[0] );
        inBuffer = (Float32 *) stream_.deviceBuffer;
      }

      if ( stream_.deviceInterleaved[0] == false ) { // mono mode
        UInt32 bufferBytes = outBufferList->mBuffers[handle->iStream[0]].mDataByteSize;
        for ( unsigned int i=0; i<stream_.nUserChannels[0]; i++ ) {
          memcpy( outBufferList->mBuffers[handle->iStream[0]+i].mData,
                  (void *)&inBuffer[i*stream_.bufferSize], bufferBytes );
        }
      }
      else { // fill multiple multi-channel streams with interleaved data
        UInt32 streamChannels, channelsLeft, inJump, outJump, inOffset;
        Float32 *out, *in;

        bool inInterleaved = ( stream_.userInterleaved ) ? true : false;
        UInt32 inChannels = stream_.nUserChannels[0];
        if ( stream_.doConvertBuffer[0] ) {
          inInterleaved = true; // device buffer will always be interleaved for nStreams > 1 and not mono mode
          inChannels = stream_.nDeviceChannels[0];
        }

        if ( inInterleaved ) inOffset = 1;
        else inOffset = stream_.bufferSize;

        channelsLeft = inChannels;
        for ( unsigned int i=0; i<handle->nStreams[0]; i++ ) {
          in = inBuffer;
          out = (Float32 *) outBufferList->mBuffers[handle->iStream[0]+i].mData;
          streamChannels = outBufferList->mBuffers[handle->iStream[0]+i].mNumberChannels;

          outJump = 0;
          // Account for possible channel offset in first stream
          if ( i == 0 && stream_.channelOffset[0] > 0 ) {
            streamChannels -= stream_.channelOffset[0];
            outJump = stream_.channelOffset[0];
            out += outJump;
          }

          // Account for possible unfilled channels at end of the last stream
          if ( streamChannels > channelsLeft ) {
            outJump = streamChannels - channelsLeft;
            streamChannels = channelsLeft;
          }

          // Determine input buffer offsets and skips
          if ( inInterleaved ) {
            inJump = inChannels;
            in += inChannels - channelsLeft;
          }
          else {
            inJump = 1;
            in += (inChannels - channelsLeft) * inOffset;
          }

          for ( unsigned int i2=0; i2<stream_.bufferSize; i2++ ) {
            for ( unsigned int j=0; j<streamChannels; j++ ) {
              *out++ = in[j*inOffset];
            }
            out += outJump;
            in += inJump;
          }
          channelsLeft -= streamChannels;
        }
      }
    }
  }

  // Don't bother draining input
  if ( handle->drainCounter ) {
    handle->drainCounter++;
    goto unlock;
  }

  AudioDeviceID inputDevice;
  inputDevice = handle->id[1];
  if ( stream_.mode == INPUT || ( stream_.mode == DUPLEX && deviceId == inputDevice ) ) {

    if ( handle->nStreams[1] == 1 ) {
      if ( stream_.doConvertBuffer[1] ) { // convert directly from CoreAudio stream buffer
        convertBuffer( stream_.userBuffer[1],
                       (char *) inBufferList->mBuffers[handle->iStream[1]].mData,
                       stream_.convertInfo[1] );
      }
      else { // copy to user buffer
        memcpy( stream_.userBuffer[1],
                inBufferList->mBuffers[handle->iStream[1]].mData,
                inBufferList->mBuffers[handle->iStream[1]].mDataByteSize );
      }
    }
    else { // read from multiple streams
      Float32 *outBuffer = (Float32 *) stream_.userBuffer[1];
      if ( stream_.doConvertBuffer[1] ) outBuffer = (Float32 *) stream_.deviceBuffer;

      if ( stream_.deviceInterleaved[1] == false ) { // mono mode
        UInt32 bufferBytes = inBufferList->mBuffers[handle->iStream[1]].mDataByteSize;
        for ( unsigned int i=0; i<stream_.nUserChannels[1]; i++ ) {
          memcpy( (void *)&outBuffer[i*stream_.bufferSize],
                  inBufferList->mBuffers[handle->iStream[1]+i].mData, bufferBytes );
        }
      }
      else { // read from multiple multi-channel streams
        UInt32 streamChannels, channelsLeft, inJump, outJump, outOffset;
        Float32 *out, *in;

        bool outInterleaved = ( stream_.userInterleaved ) ? true : false;
        UInt32 outChannels = stream_.nUserChannels[1];
        if ( stream_.doConvertBuffer[1] ) {
          outInterleaved = true; // device buffer will always be interleaved for nStreams > 1 and not mono mode
          outChannels = stream_.nDeviceChannels[1];
        }

        if ( outInterleaved ) outOffset = 1;
        else outOffset = stream_.bufferSize;

        channelsLeft = outChannels;
        for ( unsigned int i=0; i<handle->nStreams[1]; i++ ) {
          out = outBuffer;
          in = (Float32 *) inBufferList->mBuffers[handle->iStream[1]+i].mData;
          streamChannels = inBufferList->mBuffers[handle->iStream[1]+i].mNumberChannels;

          inJump = 0;
          // Account for possible channel offset in first stream
          if ( i == 0 && stream_.channelOffset[1] > 0 ) {
            streamChannels -= stream_.channelOffset[1];
            inJump = stream_.channelOffset[1];
            in += inJump;
          }

          // Account for possible unread channels at end of the last stream
          if ( streamChannels > channelsLeft ) {
            inJump = streamChannels - channelsLeft;
            streamChannels = channelsLeft;
          }

          // Determine output buffer offsets and skips
          if ( outInterleaved ) {
            outJump = outChannels;
            out += outChannels - channelsLeft;
          }
          else {
            outJump = 1;
            out += (outChannels - channelsLeft) * outOffset;
          }

          for ( unsigned int i2=0; i2<stream_.bufferSize; i2++ ) {
            for ( unsigned int j=0; j<streamChannels; j++ ) {
              out[j*outOffset] = *in++;
            }
            out += outJump;
            in += inJump;
          }
          channelsLeft -= streamChannels;
        }
      }
      
      if ( stream_.doConvertBuffer[1] ) { // convert from our internal "device" buffer
        convertBuffer( stream_.userBuffer[1],
                       stream_.deviceBuffer,
                       stream_.convertInfo[1] );
      }
    }
  }

 unlock:
  //MUTEX_UNLOCK( &stream_.mutex );

  RtApi::tickStreamTime();
  return SUCCESS;
}

const char* RtApiCore :: getErrorCode( OSStatus code )
{
  switch( code ) {

  case kAudioHardwareNotRunningError:
    return "kAudioHardwareNotRunningError";

  case kAudioHardwareUnspecifiedError:
    return "kAudioHardwareUnspecifiedError";

  case kAudioHardwareUnknownPropertyError:
    return "kAudioHardwareUnknownPropertyError";

  case kAudioHardwareBadPropertySizeError:
    return "kAudioHardwareBadPropertySizeError";

  case kAudioHardwareIllegalOperationError:
    return "kAudioHardwareIllegalOperationError";

  case kAudioHardwareBadObjectError:
    return "kAudioHardwareBadObjectError";

  case kAudioHardwareBadDeviceError:
    return "kAudioHardwareBadDeviceError";

  case kAudioHardwareBadStreamError:
    return "kAudioHardwareBadStreamError";

  case kAudioHardwareUnsupportedOperationError:
    return "kAudioHardwareUnsupportedOperationError";

  case kAudioDeviceUnsupportedFormatError:
    return "kAudioDeviceUnsupportedFormatError";

  case kAudioDevicePermissionsError:
    return "kAudioDevicePermissionsError";

  default:
    return "CoreAudio unknown error";
  }
}

  //******************** End of __MACOSX_CORE__ *********************//
#endif

#if defined(__UNIX_JACK__)

// JACK is a low-latency audio server, originally written for the
// GNU/Linux operating system and now also ported to OS-X. It can
// connect a number of different applications to an audio device, as
// well as allowing them to share audio between themselves.
//
// When using JACK with RtAudio, "devices" refer to JACK clients that
// have ports connected to the server.  The JACK server is typically
// started in a terminal as follows:
//
// .jackd -d alsa -d hw:0
//
// or through an interface program such as qjackctl.  Many of the
// parameters normally set for a stream are fixed by the JACK server
// and can be specified when the JACK server is started.  In
// particular,
//
// .jackd -d alsa -d hw:0 -r 44100 -p 512 -n 4
//
// specifies a sample rate of 44100 Hz, a buffer size of 512 sample
// frames, and number of buffers = 4.  Once the server is running, it
// is not possible to override these values.  If the values are not
// specified in the command-line, the JACK server uses default values.
//
// The JACK server does not have to be running when an instance of
// RtApiJack is created, though the function getDeviceCount() will
// report 0 devices found until JACK has been started.  When no
// devices are available (i.e., the JACK server is not running), a
// stream cannot be opened.

#include <jack/jack.h>
#include <unistd.h>
#include <cstdio>

// A structure to hold various information related to the Jack API
// implementation.
struct JackHandle {
  jack_client_t *client;
  jack_port_t **ports[2];
  std::string deviceName[2];
  bool xrun[2];
  pthread_cond_t condition;
  int drainCounter;       // Tracks callback counts when draining
  bool internalDrain;     // Indicates if stop is initiated from callback or not.

  JackHandle()
    :client(0), drainCounter(0), internalDrain(false) { ports[0] = 0; ports[1] = 0; xrun[0] = false; xrun[1] = false; }
};

#if !defined(__RTAUDIO_DEBUG__)
static void jackSilentError( const char * ) {};
#endif

RtApiJack :: RtApiJack()
    :shouldAutoconnect_(true) {
  // Nothing to do here.
#if !defined(__RTAUDIO_DEBUG__)
  // Turn off Jack's internal error reporting.
  jack_set_error_function( &jackSilentError );
#endif
}

RtApiJack :: ~RtApiJack()
{
  if ( stream_.state != STREAM_CLOSED ) closeStream();
}

unsigned int RtApiJack :: getDeviceCount( void )
{
  // See if we can become a jack client.
  jack_options_t options = (jack_options_t) ( JackNoStartServer ); //JackNullOption;
  jack_status_t *status = NULL;
  jack_client_t *client = jack_client_open( "RtApiJackCount", options, status );
  if ( client == 0 ) return 0;

  const char **ports;
  std::string port, previousPort;
  unsigned int nChannels = 0, nDevices = 0;
  ports = jack_get_ports( client, NULL, NULL, 0 );
  if ( ports ) {
    // Parse the port names up to the first colon (:).
    size_t iColon = 0;
    do {
      port = (char *) ports[ nChannels ];
      iColon = port.find(":");
      if ( iColon != std::string::npos ) {
        port = port.substr( 0, iColon + 1 );
        if ( port != previousPort ) {
          nDevices++;
          previousPort = port;
        }
      }
    } while ( ports[++nChannels] );
    free( ports );
  }

  jack_client_close( client );
  return nDevices;
}

RtAudio::DeviceInfo RtApiJack :: getDeviceInfo( unsigned int device )
{
  RtAudio::DeviceInfo info;
  info.probed = false;

  jack_options_t options = (jack_options_t) ( JackNoStartServer ); //JackNullOption
  jack_status_t *status = NULL;
  jack_client_t *client = jack_client_open( "RtApiJackInfo", options, status );
  if ( client == 0 ) {
    errorText_ = "RtApiJack::getDeviceInfo: Jack server not found or connection error!";
    error( RtAudioError::WARNING );
    return info;
  }

  const char **ports;
  std::string port, previousPort;
  unsigned int nPorts = 0, nDevices = 0;
  ports = jack_get_ports( client, NULL, NULL, 0 );
  if ( ports ) {
    // Parse the port names up to the first colon (:).
    size_t iColon = 0;
    do {
      port = (char *) ports[ nPorts ];
      iColon = port.find(":");
      if ( iColon != std::string::npos ) {
        port = port.substr( 0, iColon );
        if ( port != previousPort ) {
          if ( nDevices == device ) info.name = port;
          nDevices++;
          previousPort = port;
        }
      }
    } while ( ports[++nPorts] );
    free( ports );
  }

  if ( device >= nDevices ) {
    jack_client_close( client );
    errorText_ = "RtApiJack::getDeviceInfo: device ID is invalid!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  // Get the current jack server sample rate.
  info.sampleRates.clear();

  info.preferredSampleRate = jack_get_sample_rate( client );
  info.sampleRates.push_back( info.preferredSampleRate );

  // Count the available ports containing the client name as device
  // channels.  Jack "input ports" equal RtAudio output channels.
  unsigned int nChannels = 0;
  ports = jack_get_ports( client, info.name.c_str(), NULL, JackPortIsInput );
  if ( ports ) {
    while ( ports[ nChannels ] ) nChannels++;
    free( ports );
    info.outputChannels = nChannels;
  }

  // Jack "output ports" equal RtAudio input channels.
  nChannels = 0;
  ports = jack_get_ports( client, info.name.c_str(), NULL, JackPortIsOutput );
  if ( ports ) {
    while ( ports[ nChannels ] ) nChannels++;
    free( ports );
    info.inputChannels = nChannels;
  }

  if ( info.outputChannels == 0 && info.inputChannels == 0 ) {
    jack_client_close(client);
    errorText_ = "RtApiJack::getDeviceInfo: error determining Jack input/output channels!";
    error( RtAudioError::WARNING );
    return info;
  }

  // If device opens for both playback and capture, we determine the channels.
  if ( info.outputChannels > 0 && info.inputChannels > 0 )
    info.duplexChannels = (info.outputChannels > info.inputChannels) ? info.inputChannels : info.outputChannels;

  // Jack always uses 32-bit floats.
  info.nativeFormats = RTAUDIO_FLOAT32;

  // Jack doesn't provide default devices so we'll use the first available one.
  if ( device == 0 && info.outputChannels > 0 )
    info.isDefaultOutput = true;
  if ( device == 0 && info.inputChannels > 0 )
    info.isDefaultInput = true;

  jack_client_close(client);
  info.probed = true;
  return info;
}

static int jackCallbackHandler( jack_nframes_t nframes, void *infoPointer )
{
  CallbackInfo *info = (CallbackInfo *) infoPointer;

  RtApiJack *object = (RtApiJack *) info->object;
  if ( object->callbackEvent( (unsigned long) nframes ) == false ) return 1;

  return 0;
}

// This function will be called by a spawned thread when the Jack
// server signals that it is shutting down.  It is necessary to handle
// it this way because the jackShutdown() function must return before
// the jack_deactivate() function (in closeStream()) will return.
static void *jackCloseStream( void *ptr )
{
  CallbackInfo *info = (CallbackInfo *) ptr;
  RtApiJack *object = (RtApiJack *) info->object;

  object->closeStream();

  pthread_exit( NULL );
}
static void jackShutdown( void *infoPointer )
{
  CallbackInfo *info = (CallbackInfo *) infoPointer;
  RtApiJack *object = (RtApiJack *) info->object;

  // Check current stream state.  If stopped, then we'll assume this
  // was called as a result of a call to RtApiJack::stopStream (the
  // deactivation of a client handle causes this function to be called).
  // If not, we'll assume the Jack server is shutting down or some
  // other problem occurred and we should close the stream.
  if ( object->isStreamRunning() == false ) return;

  ThreadHandle threadId;
  pthread_create( &threadId, NULL, jackCloseStream, info );
  std::cerr << "\nRtApiJack: the Jack server is shutting down this client ... stream stopped and closed!!\n" << std::endl;
}

static int jackXrun( void *infoPointer )
{
  JackHandle *handle = (JackHandle *) infoPointer;

  if ( handle->ports[0] ) handle->xrun[0] = true;
  if ( handle->ports[1] ) handle->xrun[1] = true;

  return 0;
}

bool RtApiJack :: probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                                   unsigned int firstChannel, unsigned int sampleRate,
                                   RtAudioFormat format, unsigned int *bufferSize,
                                   RtAudio::StreamOptions *options )
{
  JackHandle *handle = (JackHandle *) stream_.apiHandle;

  // Look for jack server and try to become a client (only do once per stream).
  jack_client_t *client = 0;
  if ( mode == OUTPUT || ( mode == INPUT && stream_.mode != OUTPUT ) ) {
    jack_options_t jackoptions = (jack_options_t) ( JackNoStartServer ); //JackNullOption;
    jack_status_t *status = NULL;
    if ( options && !options->streamName.empty() )
      client = jack_client_open( options->streamName.c_str(), jackoptions, status );
    else
      client = jack_client_open( "RtApiJack", jackoptions, status );
    if ( client == 0 ) {
      errorText_ = "RtApiJack::probeDeviceOpen: Jack server not found or connection error!";
      error( RtAudioError::WARNING );
      return FAILURE;
    }
  }
  else {
    // The handle must have been created on an earlier pass.
    client = handle->client;
  }

  const char **ports;
  std::string port, previousPort, deviceName;
  unsigned int nPorts = 0, nDevices = 0;
  ports = jack_get_ports( client, NULL, NULL, 0 );
  if ( ports ) {
    // Parse the port names up to the first colon (:).
    size_t iColon = 0;
    do {
      port = (char *) ports[ nPorts ];
      iColon = port.find(":");
      if ( iColon != std::string::npos ) {
        port = port.substr( 0, iColon );
        if ( port != previousPort ) {
          if ( nDevices == device ) deviceName = port;
          nDevices++;
          previousPort = port;
        }
      }
    } while ( ports[++nPorts] );
    free( ports );
  }

  if ( device >= nDevices ) {
    errorText_ = "RtApiJack::probeDeviceOpen: device ID is invalid!";
    return FAILURE;
  }

  // Count the available ports containing the client name as device
  // channels.  Jack "input ports" equal RtAudio output channels.
  unsigned int nChannels = 0;
  unsigned long flag = JackPortIsInput;
  if ( mode == INPUT ) flag = JackPortIsOutput;
  ports = jack_get_ports( client, deviceName.c_str(), NULL, flag );
  if ( ports ) {
    while ( ports[ nChannels ] ) nChannels++;
    free( ports );
  }

  // Compare the jack ports for specified client to the requested number of channels.
  if ( nChannels < (channels + firstChannel) ) {
    errorStream_ << "RtApiJack::probeDeviceOpen: requested number of channels (" << channels << ") + offset (" << firstChannel << ") not found for specified device (" << device << ":" << deviceName << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Check the jack server sample rate.
  unsigned int jackRate = jack_get_sample_rate( client );
  if ( sampleRate != jackRate ) {
    jack_client_close( client );
    errorStream_ << "RtApiJack::probeDeviceOpen: the requested sample rate (" << sampleRate << ") is different than the JACK server rate (" << jackRate << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }
  stream_.sampleRate = jackRate;

  // Get the latency of the JACK port.
  ports = jack_get_ports( client, deviceName.c_str(), NULL, flag );
  if ( ports[ firstChannel ] ) {
    // Added by Ge Wang
    jack_latency_callback_mode_t cbmode = (mode == INPUT ? JackCaptureLatency : JackPlaybackLatency);
    // the range (usually the min and max are equal)
    jack_latency_range_t latrange; latrange.min = latrange.max = 0;
    // get the latency range
    jack_port_get_latency_range( jack_port_by_name( client, ports[firstChannel] ), cbmode, &latrange );
    // be optimistic, use the min!
    stream_.latency[mode] = latrange.min;
    //stream_.latency[mode] = jack_port_get_latency( jack_port_by_name( client, ports[ firstChannel ] ) );
  }
  free( ports );

  // The jack server always uses 32-bit floating-point data.
  stream_.deviceFormat[mode] = RTAUDIO_FLOAT32;
  stream_.userFormat = format;

  if ( options && options->flags & RTAUDIO_NONINTERLEAVED ) stream_.userInterleaved = false;
  else stream_.userInterleaved = true;

  // Jack always uses non-interleaved buffers.
  stream_.deviceInterleaved[mode] = false;

  // Jack always provides host byte-ordered data.
  stream_.doByteSwap[mode] = false;

  // Get the buffer size.  The buffer size and number of buffers
  // (periods) is set when the jack server is started.
  stream_.bufferSize = (int) jack_get_buffer_size( client );
  *bufferSize = stream_.bufferSize;

  stream_.nDeviceChannels[mode] = channels;
  stream_.nUserChannels[mode] = channels;

  // Set flags for buffer conversion.
  stream_.doConvertBuffer[mode] = false;
  if ( stream_.userFormat != stream_.deviceFormat[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.userInterleaved != stream_.deviceInterleaved[mode] &&
       stream_.nUserChannels[mode] > 1 )
    stream_.doConvertBuffer[mode] = true;

  // Allocate our JackHandle structure for the stream.
  if ( handle == 0 ) {
    try {
      handle = new JackHandle;
    }
    catch ( std::bad_alloc& ) {
      errorText_ = "RtApiJack::probeDeviceOpen: error allocating JackHandle memory.";
      goto error;
    }

    if ( pthread_cond_init(&handle->condition, NULL) ) {
      errorText_ = "RtApiJack::probeDeviceOpen: error initializing pthread condition variable.";
      goto error;
    }
    stream_.apiHandle = (void *) handle;
    handle->client = client;
  }
  handle->deviceName[mode] = deviceName;

  // Allocate necessary internal buffers.
  unsigned long bufferBytes;
  bufferBytes = stream_.nUserChannels[mode] * *bufferSize * formatBytes( stream_.userFormat );
  stream_.userBuffer[mode] = (char *) calloc( bufferBytes, 1 );
  if ( stream_.userBuffer[mode] == NULL ) {
    errorText_ = "RtApiJack::probeDeviceOpen: error allocating user buffer memory.";
    goto error;
  }

  if ( stream_.doConvertBuffer[mode] ) {

    bool makeBuffer = true;
    if ( mode == OUTPUT )
      bufferBytes = stream_.nDeviceChannels[0] * formatBytes( stream_.deviceFormat[0] );
    else { // mode == INPUT
      bufferBytes = stream_.nDeviceChannels[1] * formatBytes( stream_.deviceFormat[1] );
      if ( stream_.mode == OUTPUT && stream_.deviceBuffer ) {
        unsigned long bytesOut = stream_.nDeviceChannels[0] * formatBytes(stream_.deviceFormat[0]);
        if ( bufferBytes < bytesOut ) makeBuffer = false;
      }
    }

    if ( makeBuffer ) {
      bufferBytes *= *bufferSize;
      if ( stream_.deviceBuffer ) free( stream_.deviceBuffer );
      stream_.deviceBuffer = (char *) calloc( bufferBytes, 1 );
      if ( stream_.deviceBuffer == NULL ) {
        errorText_ = "RtApiJack::probeDeviceOpen: error allocating device buffer memory.";
        goto error;
      }
    }
  }

  // Allocate memory for the Jack ports (channels) identifiers.
  handle->ports[mode] = (jack_port_t **) malloc ( sizeof (jack_port_t *) * channels );
  if ( handle->ports[mode] == NULL )  {
    errorText_ = "RtApiJack::probeDeviceOpen: error allocating port memory.";
    goto error;
  }

  stream_.device[mode] = device;
  stream_.channelOffset[mode] = firstChannel;
  stream_.state = STREAM_STOPPED;
  stream_.callbackInfo.object = (void *) this;

  if ( stream_.mode == OUTPUT && mode == INPUT )
    // We had already set up the stream for output.
    stream_.mode = DUPLEX;
  else {
    stream_.mode = mode;
    jack_set_process_callback( handle->client, jackCallbackHandler, (void *) &stream_.callbackInfo );
    jack_set_xrun_callback( handle->client, jackXrun, (void *) &handle );
    jack_on_shutdown( handle->client, jackShutdown, (void *) &stream_.callbackInfo );
  }

  // Register our ports.
  char label[64];
  if ( mode == OUTPUT ) {
    for ( unsigned int i=0; i<stream_.nUserChannels[0]; i++ ) {
      snprintf( label, 64, "outport %d", i );
      handle->ports[0][i] = jack_port_register( handle->client, (const char *)label,
                                                JACK_DEFAULT_AUDIO_TYPE, JackPortIsOutput, 0 );
    }
  }
  else {
    for ( unsigned int i=0; i<stream_.nUserChannels[1]; i++ ) {
      snprintf( label, 64, "inport %d", i );
      handle->ports[1][i] = jack_port_register( handle->client, (const char *)label,
                                                JACK_DEFAULT_AUDIO_TYPE, JackPortIsInput, 0 );
    }
  }

  // Setup the buffer conversion information structure.  We don't use
  // buffers to do channel offsets, so we override that parameter
  // here.
  if ( stream_.doConvertBuffer[mode] ) setConvertInfo( mode, 0 );

  if ( options && options->flags & RTAUDIO_JACK_DONT_CONNECT ) shouldAutoconnect_ = false;

  return SUCCESS;

 error:
  if ( handle ) {
    pthread_cond_destroy( &handle->condition );
    jack_client_close( handle->client );

    if ( handle->ports[0] ) free( handle->ports[0] );
    if ( handle->ports[1] ) free( handle->ports[1] );

    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  return FAILURE;
}

void RtApiJack :: closeStream( void )
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiJack::closeStream(): no open stream to close!";
    error( RtAudioError::WARNING );
    return;
  }

  JackHandle *handle = (JackHandle *) stream_.apiHandle;
  if ( handle ) {

    if ( stream_.state == STREAM_RUNNING )
      jack_deactivate( handle->client );

    jack_client_close( handle->client );
  }

  if ( handle ) {
    if ( handle->ports[0] ) free( handle->ports[0] );
    if ( handle->ports[1] ) free( handle->ports[1] );
    pthread_cond_destroy( &handle->condition );
    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.mode = UNINITIALIZED;
  stream_.state = STREAM_CLOSED;
}

void RtApiJack :: startStream( void )
{
  verifyStream();
  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiJack::startStream(): the stream is already running!";
    error( RtAudioError::WARNING );
    return;
  }

  JackHandle *handle = (JackHandle *) stream_.apiHandle;
  int result = jack_activate( handle->client );
  if ( result ) {
    errorText_ = "RtApiJack::startStream(): unable to activate JACK client!";
    goto unlock;
  }

  const char **ports;

  // Get the list of available ports.
  if ( shouldAutoconnect_ && (stream_.mode == OUTPUT || stream_.mode == DUPLEX) ) {
    result = 1;
    ports = jack_get_ports( handle->client, handle->deviceName[0].c_str(), NULL, JackPortIsInput);
    if ( ports == NULL) {
      errorText_ = "RtApiJack::startStream(): error determining available JACK input ports!";
      goto unlock;
    }

    // Now make the port connections.  Since RtAudio wasn't designed to
    // allow the user to select particular channels of a device, we'll
    // just open the first "nChannels" ports with offset.
    for ( unsigned int i=0; i<stream_.nUserChannels[0]; i++ ) {
      result = 1;
      if ( ports[ stream_.channelOffset[0] + i ] )
        result = jack_connect( handle->client, jack_port_name( handle->ports[0][i] ), ports[ stream_.channelOffset[0] + i ] );
      if ( result ) {
        free( ports );
        errorText_ = "RtApiJack::startStream(): error connecting output ports!";
        goto unlock;
      }
    }
    free(ports);
  }

  if ( shouldAutoconnect_ && (stream_.mode == INPUT || stream_.mode == DUPLEX) ) {
    result = 1;
    ports = jack_get_ports( handle->client, handle->deviceName[1].c_str(), NULL, JackPortIsOutput );
    if ( ports == NULL) {
      errorText_ = "RtApiJack::startStream(): error determining available JACK output ports!";
      goto unlock;
    }

    // Now make the port connections.  See note above.
    for ( unsigned int i=0; i<stream_.nUserChannels[1]; i++ ) {
      result = 1;
      if ( ports[ stream_.channelOffset[1] + i ] )
        result = jack_connect( handle->client, ports[ stream_.channelOffset[1] + i ], jack_port_name( handle->ports[1][i] ) );
      if ( result ) {
        free( ports );
        errorText_ = "RtApiJack::startStream(): error connecting input ports!";
        goto unlock;
      }
    }
    free(ports);
  }

  handle->drainCounter = 0;
  handle->internalDrain = false;
  stream_.state = STREAM_RUNNING;

 unlock:
  if ( result == 0 ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiJack :: stopStream( void )
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiJack::stopStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  JackHandle *handle = (JackHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    if ( handle->drainCounter == 0 ) {
      handle->drainCounter = 2;
      pthread_cond_wait( &handle->condition, &stream_.mutex ); // block until signaled
    }
  }

  jack_deactivate( handle->client );
  stream_.state = STREAM_STOPPED;
}

void RtApiJack :: abortStream( void )
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiJack::abortStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  JackHandle *handle = (JackHandle *) stream_.apiHandle;
  handle->drainCounter = 2;

  stopStream();
}

// This function will be called by a spawned thread when the user
// callback function signals that the stream should be stopped or
// aborted.  It is necessary to handle it this way because the
// callbackEvent() function must return before the jack_deactivate()
// function will return.
static void *jackStopStream( void *ptr )
{
  CallbackInfo *info = (CallbackInfo *) ptr;
  RtApiJack *object = (RtApiJack *) info->object;

  object->stopStream();
  pthread_exit( NULL );
}

bool RtApiJack :: callbackEvent( unsigned long nframes )
{
  if ( stream_.state == STREAM_STOPPED || stream_.state == STREAM_STOPPING ) return SUCCESS;
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiCore::callbackEvent(): the stream is closed ... this shouldn't happen!";
    error( RtAudioError::WARNING );
    return FAILURE;
  }
  if ( stream_.bufferSize != nframes ) {
    errorText_ = "RtApiCore::callbackEvent(): the JACK buffer size has changed ... cannot process!";
    error( RtAudioError::WARNING );
    return FAILURE;
  }

  CallbackInfo *info = (CallbackInfo *) &stream_.callbackInfo;
  JackHandle *handle = (JackHandle *) stream_.apiHandle;

  // Check if we were draining the stream and signal is finished.
  if ( handle->drainCounter > 3 ) {
    ThreadHandle threadId;

    stream_.state = STREAM_STOPPING;
    if ( handle->internalDrain == true )
      pthread_create( &threadId, NULL, jackStopStream, info );
    else
      pthread_cond_signal( &handle->condition );
    return SUCCESS;
  }

  // Invoke user callback first, to get fresh output data.
  if ( handle->drainCounter == 0 ) {
    RtAudioCallback callback = (RtAudioCallback) info->callback;
    double streamTime = getStreamTime();
    RtAudioStreamStatus status = 0;
    if ( stream_.mode != INPUT && handle->xrun[0] == true ) {
      status |= RTAUDIO_OUTPUT_UNDERFLOW;
      handle->xrun[0] = false;
    }
    if ( stream_.mode != OUTPUT && handle->xrun[1] == true ) {
      status |= RTAUDIO_INPUT_OVERFLOW;
      handle->xrun[1] = false;
    }
    int cbReturnValue = callback( stream_.userBuffer[0], stream_.userBuffer[1],
                                  stream_.bufferSize, streamTime, status, info->userData );
    if ( cbReturnValue == 2 ) {
      stream_.state = STREAM_STOPPING;
      handle->drainCounter = 2;
      ThreadHandle id;
      pthread_create( &id, NULL, jackStopStream, info );
      return SUCCESS;
    }
    else if ( cbReturnValue == 1 ) {
      handle->drainCounter = 1;
      handle->internalDrain = true;
    }
  }

  jack_default_audio_sample_t *jackbuffer;
  unsigned long bufferBytes = nframes * sizeof( jack_default_audio_sample_t );
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    if ( handle->drainCounter > 1 ) { // write zeros to the output stream

      for ( unsigned int i=0; i<stream_.nDeviceChannels[0]; i++ ) {
        jackbuffer = (jack_default_audio_sample_t *) jack_port_get_buffer( handle->ports[0][i], (jack_nframes_t) nframes );
        memset( jackbuffer, 0, bufferBytes );
      }

    }
    else if ( stream_.doConvertBuffer[0] ) {

      convertBuffer( stream_.deviceBuffer, stream_.userBuffer[0], stream_.convertInfo[0] );

      for ( unsigned int i=0; i<stream_.nDeviceChannels[0]; i++ ) {
        jackbuffer = (jack_default_audio_sample_t *) jack_port_get_buffer( handle->ports[0][i], (jack_nframes_t) nframes );
        memcpy( jackbuffer, &stream_.deviceBuffer[i*bufferBytes], bufferBytes );
      }
    }
    else { // no buffer conversion
      for ( unsigned int i=0; i<stream_.nUserChannels[0]; i++ ) {
        jackbuffer = (jack_default_audio_sample_t *) jack_port_get_buffer( handle->ports[0][i], (jack_nframes_t) nframes );
        memcpy( jackbuffer, &stream_.userBuffer[0][i*bufferBytes], bufferBytes );
      }
    }
  }

  // Don't bother draining input
  if ( handle->drainCounter ) {
    handle->drainCounter++;
    goto unlock;
  }

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX ) {

    if ( stream_.doConvertBuffer[1] ) {
      for ( unsigned int i=0; i<stream_.nDeviceChannels[1]; i++ ) {
        jackbuffer = (jack_default_audio_sample_t *) jack_port_get_buffer( handle->ports[1][i], (jack_nframes_t) nframes );
        memcpy( &stream_.deviceBuffer[i*bufferBytes], jackbuffer, bufferBytes );
      }
      convertBuffer( stream_.userBuffer[1], stream_.deviceBuffer, stream_.convertInfo[1] );
    }
    else { // no buffer conversion
      for ( unsigned int i=0; i<stream_.nUserChannels[1]; i++ ) {
        jackbuffer = (jack_default_audio_sample_t *) jack_port_get_buffer( handle->ports[1][i], (jack_nframes_t) nframes );
        memcpy( &stream_.userBuffer[1][i*bufferBytes], jackbuffer, bufferBytes );
      }
    }
  }

 unlock:
  RtApi::tickStreamTime();
  return SUCCESS;
}
  //******************** End of __UNIX_JACK__ *********************//
#endif

#if defined(__WINDOWS_ASIO__) // ASIO API on Windows

// The ASIO API is designed around a callback scheme, so this
// implementation is similar to that used for OS-X CoreAudio and Linux
// Jack.  The primary constraint with ASIO is that it only allows
// access to a single driver at a time.  Thus, it is not possible to
// have more than one simultaneous RtAudio stream.
//
// This implementation also requires a number of external ASIO files
// and a few global variables.  The ASIO callback scheme does not
// allow for the passing of user data, so we must create a global
// pointer to our callbackInfo structure.
//
// On unix systems, we make use of a pthread condition variable.
// Since there is no equivalent in Windows, I hacked something based
// on information found in
// http://www.cs.wustl.edu/~schmidt/win32-cv-1.html.

#include "asiosys.h"
#include "asio.h"
#include "iasiothiscallresolver.h"
#include "asiodrivers.h"
#include <cmath>

static AsioDrivers drivers;
static ASIOCallbacks asioCallbacks;
static ASIODriverInfo driverInfo;
static CallbackInfo *asioCallbackInfo;
static bool asioXRun;

struct AsioHandle {
  int drainCounter;       // Tracks callback counts when draining
  bool internalDrain;     // Indicates if stop is initiated from callback or not.
  ASIOBufferInfo *bufferInfos;
  HANDLE condition;

  AsioHandle()
    :drainCounter(0), internalDrain(false), bufferInfos(0) {}
};

// Function declarations (definitions at end of section)
static const char* getAsioErrorString( ASIOError result );
static void sampleRateChanged( ASIOSampleRate sRate );
static long asioMessages( long selector, long value, void* message, double* opt );

RtApiAsio :: RtApiAsio()
{
  // ASIO cannot run on a multi-threaded appartment. You can call
  // CoInitialize beforehand, but it must be for appartment threading
  // (in which case, CoInitilialize will return S_FALSE here).
  coInitialized_ = false;
  HRESULT hr = CoInitialize( NULL ); 
  if ( FAILED(hr) ) {
    errorText_ = "RtApiAsio::ASIO requires a single-threaded appartment. Call CoInitializeEx(0,COINIT_APARTMENTTHREADED)";
    error( RtAudioError::WARNING );
  }
  coInitialized_ = true;

  drivers.removeCurrentDriver();
  driverInfo.asioVersion = 2;

  // See note in DirectSound implementation about GetDesktopWindow().
  driverInfo.sysRef = GetForegroundWindow();
}

RtApiAsio :: ~RtApiAsio()
{
  if ( stream_.state != STREAM_CLOSED ) closeStream();
  if ( coInitialized_ ) CoUninitialize();
}

unsigned int RtApiAsio :: getDeviceCount( void )
{
  return (unsigned int) drivers.asioGetNumDev();
}

RtAudio::DeviceInfo RtApiAsio :: getDeviceInfo( unsigned int device )
{
  RtAudio::DeviceInfo info;
  info.probed = false;

  // Get device ID
  unsigned int nDevices = getDeviceCount();
  if ( nDevices == 0 ) {
    errorText_ = "RtApiAsio::getDeviceInfo: no devices found!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  if ( device >= nDevices ) {
    errorText_ = "RtApiAsio::getDeviceInfo: device ID is invalid!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  // If a stream is already open, we cannot probe other devices.  Thus, use the saved results.
  if ( stream_.state != STREAM_CLOSED ) {
    if ( device >= devices_.size() ) {
      errorText_ = "RtApiAsio::getDeviceInfo: device ID was not present before stream was opened.";
      error( RtAudioError::WARNING );
      return info;
    }
    return devices_[ device ];
  }

  char driverName[32];
  ASIOError result = drivers.asioGetDriverName( (int) device, driverName, 32 );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::getDeviceInfo: unable to get driver name (" << getAsioErrorString( result ) << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  info.name = driverName;

  if ( !drivers.loadDriver( driverName ) ) {
    errorStream_ << "RtApiAsio::getDeviceInfo: unable to load driver (" << driverName << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  result = ASIOInit( &driverInfo );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::getDeviceInfo: error (" << getAsioErrorString( result ) << ") initializing driver (" << driverName << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Determine the device channel information.
  long inputChannels, outputChannels;
  result = ASIOGetChannels( &inputChannels, &outputChannels );
  if ( result != ASE_OK ) {
    drivers.removeCurrentDriver();
    errorStream_ << "RtApiAsio::getDeviceInfo: error (" << getAsioErrorString( result ) << ") getting channel count (" << driverName << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  info.outputChannels = outputChannels;
  info.inputChannels = inputChannels;
  if ( info.outputChannels > 0 && info.inputChannels > 0 )
    info.duplexChannels = (info.outputChannels > info.inputChannels) ? info.inputChannels : info.outputChannels;

  // Determine the supported sample rates.
  info.sampleRates.clear();
  for ( unsigned int i=0; i<MAX_SAMPLE_RATES; i++ ) {
    result = ASIOCanSampleRate( (ASIOSampleRate) SAMPLE_RATES[i] );
    if ( result == ASE_OK ) {
      info.sampleRates.push_back( SAMPLE_RATES[i] );

      if ( !info.preferredSampleRate || ( SAMPLE_RATES[i] <= 48000 && SAMPLE_RATES[i] > info.preferredSampleRate ) )
        info.preferredSampleRate = SAMPLE_RATES[i];
    }
  }

  // Determine supported data types ... just check first channel and assume rest are the same.
  ASIOChannelInfo channelInfo;
  channelInfo.channel = 0;
  channelInfo.isInput = true;
  if ( info.inputChannels <= 0 ) channelInfo.isInput = false;
  result = ASIOGetChannelInfo( &channelInfo );
  if ( result != ASE_OK ) {
    drivers.removeCurrentDriver();
    errorStream_ << "RtApiAsio::getDeviceInfo: error (" << getAsioErrorString( result ) << ") getting driver channel info (" << driverName << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  info.nativeFormats = 0;
  if ( channelInfo.type == ASIOSTInt16MSB || channelInfo.type == ASIOSTInt16LSB )
    info.nativeFormats |= RTAUDIO_SINT16;
  else if ( channelInfo.type == ASIOSTInt32MSB || channelInfo.type == ASIOSTInt32LSB )
    info.nativeFormats |= RTAUDIO_SINT32;
  else if ( channelInfo.type == ASIOSTFloat32MSB || channelInfo.type == ASIOSTFloat32LSB )
    info.nativeFormats |= RTAUDIO_FLOAT32;
  else if ( channelInfo.type == ASIOSTFloat64MSB || channelInfo.type == ASIOSTFloat64LSB )
    info.nativeFormats |= RTAUDIO_FLOAT64;
  else if ( channelInfo.type == ASIOSTInt24MSB || channelInfo.type == ASIOSTInt24LSB )
    info.nativeFormats |= RTAUDIO_SINT24;

  if ( info.outputChannels > 0 )
    if ( getDefaultOutputDevice() == device ) info.isDefaultOutput = true;
  if ( info.inputChannels > 0 )
    if ( getDefaultInputDevice() == device ) info.isDefaultInput = true;

  info.probed = true;
  drivers.removeCurrentDriver();
  return info;
}

static void bufferSwitch( long index, ASIOBool /*processNow*/ )
{
  RtApiAsio *object = (RtApiAsio *) asioCallbackInfo->object;
  object->callbackEvent( index );
}

void RtApiAsio :: saveDeviceInfo( void )
{
  devices_.clear();

  unsigned int nDevices = getDeviceCount();
  devices_.resize( nDevices );
  for ( unsigned int i=0; i<nDevices; i++ )
    devices_[i] = getDeviceInfo( i );
}

bool RtApiAsio :: probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                                   unsigned int firstChannel, unsigned int sampleRate,
                                   RtAudioFormat format, unsigned int *bufferSize,
                                   RtAudio::StreamOptions *options )
{////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

  bool isDuplexInput =  mode == INPUT && stream_.mode == OUTPUT;

  // For ASIO, a duplex stream MUST use the same driver.
  if ( isDuplexInput && stream_.device[0] != device ) {
    errorText_ = "RtApiAsio::probeDeviceOpen: an ASIO duplex stream must use the same device for input and output!";
    return FAILURE;
  }

  char driverName[32];
  ASIOError result = drivers.asioGetDriverName( (int) device, driverName, 32 );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: unable to get driver name (" << getAsioErrorString( result ) << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Only load the driver once for duplex stream.
  if ( !isDuplexInput ) {
    // The getDeviceInfo() function will not work when a stream is open
    // because ASIO does not allow multiple devices to run at the same
    // time.  Thus, we'll probe the system before opening a stream and
    // save the results for use by getDeviceInfo().
    this->saveDeviceInfo();

    if ( !drivers.loadDriver( driverName ) ) {
      errorStream_ << "RtApiAsio::probeDeviceOpen: unable to load driver (" << driverName << ").";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    result = ASIOInit( &driverInfo );
    if ( result != ASE_OK ) {
      errorStream_ << "RtApiAsio::probeDeviceOpen: error (" << getAsioErrorString( result ) << ") initializing driver (" << driverName << ").";
      errorText_ = errorStream_.str();
      return FAILURE;
    }
  }

  // keep them before any "goto error", they are used for error cleanup + goto device boundary checks
  bool buffersAllocated = false;
  AsioHandle *handle = (AsioHandle *) stream_.apiHandle;
  unsigned int nChannels;


  // Check the device channel count.
  long inputChannels, outputChannels;
  result = ASIOGetChannels( &inputChannels, &outputChannels );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: error (" << getAsioErrorString( result ) << ") getting channel count (" << driverName << ").";
    errorText_ = errorStream_.str();
    goto error;
  }

  if ( ( mode == OUTPUT && (channels+firstChannel) > (unsigned int) outputChannels) ||
       ( mode == INPUT && (channels+firstChannel) > (unsigned int) inputChannels) ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") does not support requested channel count (" << channels << ") + offset (" << firstChannel << ").";
    errorText_ = errorStream_.str();
    goto error;
  }
  stream_.nDeviceChannels[mode] = channels;
  stream_.nUserChannels[mode] = channels;
  stream_.channelOffset[mode] = firstChannel;

  // Verify the sample rate is supported.
  result = ASIOCanSampleRate( (ASIOSampleRate) sampleRate );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") does not support requested sample rate (" << sampleRate << ").";
    errorText_ = errorStream_.str();
    goto error;
  }

  // Get the current sample rate
  ASIOSampleRate currentRate;
  result = ASIOGetSampleRate( &currentRate );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") error getting sample rate.";
    errorText_ = errorStream_.str();
    goto error;
  }

  // Set the sample rate only if necessary
  if ( currentRate != sampleRate ) {
    result = ASIOSetSampleRate( (ASIOSampleRate) sampleRate );
    if ( result != ASE_OK ) {
      errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") error setting sample rate (" << sampleRate << ").";
      errorText_ = errorStream_.str();
      goto error;
    }
  }

  // Determine the driver data type.
  ASIOChannelInfo channelInfo;
  channelInfo.channel = 0;
  if ( mode == OUTPUT ) channelInfo.isInput = false;
  else channelInfo.isInput = true;
  result = ASIOGetChannelInfo( &channelInfo );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") error (" << getAsioErrorString( result ) << ") getting data format.";
    errorText_ = errorStream_.str();
    goto error;
  }

  // Assuming WINDOWS host is always little-endian.
  stream_.doByteSwap[mode] = false;
  stream_.userFormat = format;
  stream_.deviceFormat[mode] = 0;
  if ( channelInfo.type == ASIOSTInt16MSB || channelInfo.type == ASIOSTInt16LSB ) {
    stream_.deviceFormat[mode] = RTAUDIO_SINT16;
    if ( channelInfo.type == ASIOSTInt16MSB ) stream_.doByteSwap[mode] = true;
  }
  else if ( channelInfo.type == ASIOSTInt32MSB || channelInfo.type == ASIOSTInt32LSB ) {
    stream_.deviceFormat[mode] = RTAUDIO_SINT32;
    if ( channelInfo.type == ASIOSTInt32MSB ) stream_.doByteSwap[mode] = true;
  }
  else if ( channelInfo.type == ASIOSTFloat32MSB || channelInfo.type == ASIOSTFloat32LSB ) {
    stream_.deviceFormat[mode] = RTAUDIO_FLOAT32;
    if ( channelInfo.type == ASIOSTFloat32MSB ) stream_.doByteSwap[mode] = true;
  }
  else if ( channelInfo.type == ASIOSTFloat64MSB || channelInfo.type == ASIOSTFloat64LSB ) {
    stream_.deviceFormat[mode] = RTAUDIO_FLOAT64;
    if ( channelInfo.type == ASIOSTFloat64MSB ) stream_.doByteSwap[mode] = true;
  }
  else if ( channelInfo.type == ASIOSTInt24MSB || channelInfo.type == ASIOSTInt24LSB ) {
    stream_.deviceFormat[mode] = RTAUDIO_SINT24;
    if ( channelInfo.type == ASIOSTInt24MSB ) stream_.doByteSwap[mode] = true;
  }

  if ( stream_.deviceFormat[mode] == 0 ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") data format not supported by RtAudio.";
    errorText_ = errorStream_.str();
    goto error;
  }

  // Set the buffer size.  For a duplex stream, this will end up
  // setting the buffer size based on the input constraints, which
  // should be ok.
  long minSize, maxSize, preferSize, granularity;
  result = ASIOGetBufferSize( &minSize, &maxSize, &preferSize, &granularity );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") error (" << getAsioErrorString( result ) << ") getting buffer size.";
    errorText_ = errorStream_.str();
    goto error;
  }

  if ( isDuplexInput ) {
    // When this is the duplex input (output was opened before), then we have to use the same
    // buffersize as the output, because it might use the preferred buffer size, which most
    // likely wasn't passed as input to this. The buffer sizes have to be identically anyway,
    // So instead of throwing an error, make them equal. The caller uses the reference
    // to the "bufferSize" param as usual to set up processing buffers.

    *bufferSize = stream_.bufferSize;

  } else {
    if ( *bufferSize == 0 ) *bufferSize = preferSize;
    else if ( *bufferSize < (unsigned int) minSize ) *bufferSize = (unsigned int) minSize;
    else if ( *bufferSize > (unsigned int) maxSize ) *bufferSize = (unsigned int) maxSize;
    else if ( granularity == -1 ) {
      // Make sure bufferSize is a power of two.
      int log2_of_min_size = 0;
      int log2_of_max_size = 0;

      for ( unsigned int i = 0; i < sizeof(long) * 8; i++ ) {
        if ( minSize & ((long)1 << i) ) log2_of_min_size = i;
        if ( maxSize & ((long)1 << i) ) log2_of_max_size = i;
      }

      long min_delta = std::abs( (long)*bufferSize - ((long)1 << log2_of_min_size) );
      int min_delta_num = log2_of_min_size;

      for (int i = log2_of_min_size + 1; i <= log2_of_max_size; i++) {
        long current_delta = std::abs( (long)*bufferSize - ((long)1 << i) );
        if (current_delta < min_delta) {
          min_delta = current_delta;
          min_delta_num = i;
        }
      }

      *bufferSize = ( (unsigned int)1 << min_delta_num );
      if ( *bufferSize < (unsigned int) minSize ) *bufferSize = (unsigned int) minSize;
      else if ( *bufferSize > (unsigned int) maxSize ) *bufferSize = (unsigned int) maxSize;
    }
    else if ( granularity != 0 ) {
      // Set to an even multiple of granularity, rounding up.
      *bufferSize = (*bufferSize + granularity-1) / granularity * granularity;
    }
  }

  /*
  // we don't use it anymore, see above!
  // Just left it here for the case...
  if ( isDuplexInput && stream_.bufferSize != *bufferSize ) {
    errorText_ = "RtApiAsio::probeDeviceOpen: input/output buffersize discrepancy!";
    goto error;
  }
  */

  stream_.bufferSize = *bufferSize;
  stream_.nBuffers = 2;

  if ( options && options->flags & RTAUDIO_NONINTERLEAVED ) stream_.userInterleaved = false;
  else stream_.userInterleaved = true;

  // ASIO always uses non-interleaved buffers.
  stream_.deviceInterleaved[mode] = false;

  // Allocate, if necessary, our AsioHandle structure for the stream.
  if ( handle == 0 ) {
    try {
      handle = new AsioHandle;
    }
    catch ( std::bad_alloc& ) {
      errorText_ = "RtApiAsio::probeDeviceOpen: error allocating AsioHandle memory.";
      goto error;
    }
    handle->bufferInfos = 0;

    // Create a manual-reset event.
    handle->condition = CreateEvent( NULL,   // no security
                                     TRUE,   // manual-reset
                                     FALSE,  // non-signaled initially
                                     NULL ); // unnamed
    stream_.apiHandle = (void *) handle;
  }

  // Create the ASIO internal buffers.  Since RtAudio sets up input
  // and output separately, we'll have to dispose of previously
  // created output buffers for a duplex stream.
  if ( mode == INPUT && stream_.mode == OUTPUT ) {
    ASIODisposeBuffers();
    if ( handle->bufferInfos ) free( handle->bufferInfos );
  }

  // Allocate, initialize, and save the bufferInfos in our stream callbackInfo structure.
  unsigned int i;
  nChannels = stream_.nDeviceChannels[0] + stream_.nDeviceChannels[1];
  handle->bufferInfos = (ASIOBufferInfo *) malloc( nChannels * sizeof(ASIOBufferInfo) );
  if ( handle->bufferInfos == NULL ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: error allocating bufferInfo memory for driver (" << driverName << ").";
    errorText_ = errorStream_.str();
    goto error;
  }

  ASIOBufferInfo *infos;
  infos = handle->bufferInfos;
  for ( i=0; i<stream_.nDeviceChannels[0]; i++, infos++ ) {
    infos->isInput = ASIOFalse;
    infos->channelNum = i + stream_.channelOffset[0];
    infos->buffers[0] = infos->buffers[1] = 0;
  }
  for ( i=0; i<stream_.nDeviceChannels[1]; i++, infos++ ) {
    infos->isInput = ASIOTrue;
    infos->channelNum = i + stream_.channelOffset[1];
    infos->buffers[0] = infos->buffers[1] = 0;
  }

  // prepare for callbacks
  stream_.sampleRate = sampleRate;
  stream_.device[mode] = device;
  stream_.mode = isDuplexInput ? DUPLEX : mode;

  // store this class instance before registering callbacks, that are going to use it
  asioCallbackInfo = &stream_.callbackInfo;
  stream_.callbackInfo.object = (void *) this;

  // Set up the ASIO callback structure and create the ASIO data buffers.
  asioCallbacks.bufferSwitch = &bufferSwitch;
  asioCallbacks.sampleRateDidChange = &sampleRateChanged;
  asioCallbacks.asioMessage = &asioMessages;
  asioCallbacks.bufferSwitchTimeInfo = NULL;
  result = ASIOCreateBuffers( handle->bufferInfos, nChannels, stream_.bufferSize, &asioCallbacks );
  if ( result != ASE_OK ) {
    // Standard method failed. This can happen with strict/misbehaving drivers that return valid buffer size ranges
    // but only accept the preferred buffer size as parameter for ASIOCreateBuffers. eg. Creatives ASIO driver
    // in that case, let's be naÃƒÂ¯ve and try that instead
    *bufferSize = preferSize;
    stream_.bufferSize = *bufferSize;
    result = ASIOCreateBuffers( handle->bufferInfos, nChannels, stream_.bufferSize, &asioCallbacks );
  }

  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") error (" << getAsioErrorString( result ) << ") creating buffers.";
    errorText_ = errorStream_.str();
    goto error;
  }
  buffersAllocated = true;  
  stream_.state = STREAM_STOPPED;

  // Set flags for buffer conversion.
  stream_.doConvertBuffer[mode] = false;
  if ( stream_.userFormat != stream_.deviceFormat[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.userInterleaved != stream_.deviceInterleaved[mode] &&
       stream_.nUserChannels[mode] > 1 )
    stream_.doConvertBuffer[mode] = true;

  // Allocate necessary internal buffers
  unsigned long bufferBytes;
  bufferBytes = stream_.nUserChannels[mode] * *bufferSize * formatBytes( stream_.userFormat );
  stream_.userBuffer[mode] = (char *) calloc( bufferBytes, 1 );
  if ( stream_.userBuffer[mode] == NULL ) {
    errorText_ = "RtApiAsio::probeDeviceOpen: error allocating user buffer memory.";
    goto error;
  }

  if ( stream_.doConvertBuffer[mode] ) {

    bool makeBuffer = true;
    bufferBytes = stream_.nDeviceChannels[mode] * formatBytes( stream_.deviceFormat[mode] );
    if ( isDuplexInput && stream_.deviceBuffer ) {
      unsigned long bytesOut = stream_.nDeviceChannels[0] * formatBytes( stream_.deviceFormat[0] );
      if ( bufferBytes <= bytesOut ) makeBuffer = false;
    }

    if ( makeBuffer ) {
      bufferBytes *= *bufferSize;
      if ( stream_.deviceBuffer ) free( stream_.deviceBuffer );
      stream_.deviceBuffer = (char *) calloc( bufferBytes, 1 );
      if ( stream_.deviceBuffer == NULL ) {
        errorText_ = "RtApiAsio::probeDeviceOpen: error allocating device buffer memory.";
        goto error;
      }
    }
  }

  // Determine device latencies
  long inputLatency, outputLatency;
  result = ASIOGetLatencies( &inputLatency, &outputLatency );
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::probeDeviceOpen: driver (" << driverName << ") error (" << getAsioErrorString( result ) << ") getting latency.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING); // warn but don't fail
  }
  else {
    stream_.latency[0] = outputLatency;
    stream_.latency[1] = inputLatency;
  }

  // Setup the buffer conversion information structure.  We don't use
  // buffers to do channel offsets, so we override that parameter
  // here.
  if ( stream_.doConvertBuffer[mode] ) setConvertInfo( mode, 0 );

  return SUCCESS;

 error:
  if ( !isDuplexInput ) {
    // the cleanup for error in the duplex input, is done by RtApi::openStream
    // So we clean up for single channel only

    if ( buffersAllocated )
      ASIODisposeBuffers();

    drivers.removeCurrentDriver();

    if ( handle ) {
      CloseHandle( handle->condition );
      if ( handle->bufferInfos )
        free( handle->bufferInfos );

      delete handle;
      stream_.apiHandle = 0;
    }


    if ( stream_.userBuffer[mode] ) {
      free( stream_.userBuffer[mode] );
      stream_.userBuffer[mode] = 0;
    }

    if ( stream_.deviceBuffer ) {
      free( stream_.deviceBuffer );
      stream_.deviceBuffer = 0;
    }
  }

  return FAILURE;
}////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void RtApiAsio :: closeStream()
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiAsio::closeStream(): no open stream to close!";
    error( RtAudioError::WARNING );
    return;
  }

  if ( stream_.state == STREAM_RUNNING ) {
    stream_.state = STREAM_STOPPED;
    ASIOStop();
  }
  ASIODisposeBuffers();
  drivers.removeCurrentDriver();

  AsioHandle *handle = (AsioHandle *) stream_.apiHandle;
  if ( handle ) {
    CloseHandle( handle->condition );
    if ( handle->bufferInfos )
      free( handle->bufferInfos );
    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.mode = UNINITIALIZED;
  stream_.state = STREAM_CLOSED;
}

bool stopThreadCalled = false;

void RtApiAsio :: startStream()
{
  verifyStream();
  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiAsio::startStream(): the stream is already running!";
    error( RtAudioError::WARNING );
    return;
  }

  AsioHandle *handle = (AsioHandle *) stream_.apiHandle;
  ASIOError result = ASIOStart();
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::startStream: error (" << getAsioErrorString( result ) << ") starting device.";
    errorText_ = errorStream_.str();
    goto unlock;
  }

  handle->drainCounter = 0;
  handle->internalDrain = false;
  ResetEvent( handle->condition );
  stream_.state = STREAM_RUNNING;
  asioXRun = false;

 unlock:
  stopThreadCalled = false;

  if ( result == ASE_OK ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiAsio :: stopStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiAsio::stopStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  AsioHandle *handle = (AsioHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    if ( handle->drainCounter == 0 ) {
      handle->drainCounter = 2;
      WaitForSingleObject( handle->condition, INFINITE );  // block until signaled
    }
  }

  stream_.state = STREAM_STOPPED;

  ASIOError result = ASIOStop();
  if ( result != ASE_OK ) {
    errorStream_ << "RtApiAsio::stopStream: error (" << getAsioErrorString( result ) << ") stopping device.";
    errorText_ = errorStream_.str();
  }

  if ( result == ASE_OK ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiAsio :: abortStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiAsio::abortStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  // The following lines were commented-out because some behavior was
  // noted where the device buffers need to be zeroed to avoid
  // continuing sound, even when the device buffers are completely
  // disposed.  So now, calling abort is the same as calling stop.
  // AsioHandle *handle = (AsioHandle *) stream_.apiHandle;
  // handle->drainCounter = 2;
  stopStream();
}

// This function will be called by a spawned thread when the user
// callback function signals that the stream should be stopped or
// aborted.  It is necessary to handle it this way because the
// callbackEvent() function must return before the ASIOStop()
// function will return.
static unsigned __stdcall asioStopStream( void *ptr )
{
  CallbackInfo *info = (CallbackInfo *) ptr;
  RtApiAsio *object = (RtApiAsio *) info->object;

  object->stopStream();
  _endthreadex( 0 );
  return 0;
}

bool RtApiAsio :: callbackEvent( long bufferIndex )
{
  if ( stream_.state == STREAM_STOPPED || stream_.state == STREAM_STOPPING ) return SUCCESS;
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiAsio::callbackEvent(): the stream is closed ... this shouldn't happen!";
    error( RtAudioError::WARNING );
    return FAILURE;
  }

  CallbackInfo *info = (CallbackInfo *) &stream_.callbackInfo;
  AsioHandle *handle = (AsioHandle *) stream_.apiHandle;

  // Check if we were draining the stream and signal if finished.
  if ( handle->drainCounter > 3 ) {

    stream_.state = STREAM_STOPPING;
    if ( handle->internalDrain == false )
      SetEvent( handle->condition );
    else { // spawn a thread to stop the stream
      unsigned threadId;
      stream_.callbackInfo.thread = _beginthreadex( NULL, 0, &asioStopStream,
                                                    &stream_.callbackInfo, 0, &threadId );
    }
    return SUCCESS;
  }

  // Invoke user callback to get fresh output data UNLESS we are
  // draining stream.
  if ( handle->drainCounter == 0 ) {
    RtAudioCallback callback = (RtAudioCallback) info->callback;
    double streamTime = getStreamTime();
    RtAudioStreamStatus status = 0;
    if ( stream_.mode != INPUT && asioXRun == true ) {
      status |= RTAUDIO_OUTPUT_UNDERFLOW;
      asioXRun = false;
    }
    if ( stream_.mode != OUTPUT && asioXRun == true ) {
      status |= RTAUDIO_INPUT_OVERFLOW;
      asioXRun = false;
    }
    int cbReturnValue = callback( stream_.userBuffer[0], stream_.userBuffer[1],
                                     stream_.bufferSize, streamTime, status, info->userData );
    if ( cbReturnValue == 2 ) {
      stream_.state = STREAM_STOPPING;
      handle->drainCounter = 2;
      unsigned threadId;
      stream_.callbackInfo.thread = _beginthreadex( NULL, 0, &asioStopStream,
                                                    &stream_.callbackInfo, 0, &threadId );
      return SUCCESS;
    }
    else if ( cbReturnValue == 1 ) {
      handle->drainCounter = 1;
      handle->internalDrain = true;
    }
  }

  unsigned int nChannels, bufferBytes, i, j;
  nChannels = stream_.nDeviceChannels[0] + stream_.nDeviceChannels[1];
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    bufferBytes = stream_.bufferSize * formatBytes( stream_.deviceFormat[0] );

    if ( handle->drainCounter > 1 ) { // write zeros to the output stream

      for ( i=0, j=0; i<nChannels; i++ ) {
        if ( handle->bufferInfos[i].isInput != ASIOTrue )
          memset( handle->bufferInfos[i].buffers[bufferIndex], 0, bufferBytes );
      }

    }
    else if ( stream_.doConvertBuffer[0] ) {

      convertBuffer( stream_.deviceBuffer, stream_.userBuffer[0], stream_.convertInfo[0] );
      if ( stream_.doByteSwap[0] )
        byteSwapBuffer( stream_.deviceBuffer,
                        stream_.bufferSize * stream_.nDeviceChannels[0],
                        stream_.deviceFormat[0] );

      for ( i=0, j=0; i<nChannels; i++ ) {
        if ( handle->bufferInfos[i].isInput != ASIOTrue )
          memcpy( handle->bufferInfos[i].buffers[bufferIndex],
                  &stream_.deviceBuffer[j++*bufferBytes], bufferBytes );
      }

    }
    else {

      if ( stream_.doByteSwap[0] )
        byteSwapBuffer( stream_.userBuffer[0],
                        stream_.bufferSize * stream_.nUserChannels[0],
                        stream_.userFormat );

      for ( i=0, j=0; i<nChannels; i++ ) {
        if ( handle->bufferInfos[i].isInput != ASIOTrue )
          memcpy( handle->bufferInfos[i].buffers[bufferIndex],
                  &stream_.userBuffer[0][bufferBytes*j++], bufferBytes );
      }

    }
  }

  // Don't bother draining input
  if ( handle->drainCounter ) {
    handle->drainCounter++;
    goto unlock;
  }

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX ) {

    bufferBytes = stream_.bufferSize * formatBytes(stream_.deviceFormat[1]);

    if (stream_.doConvertBuffer[1]) {

      // Always interleave ASIO input data.
      for ( i=0, j=0; i<nChannels; i++ ) {
        if ( handle->bufferInfos[i].isInput == ASIOTrue )
          memcpy( &stream_.deviceBuffer[j++*bufferBytes],
                  handle->bufferInfos[i].buffers[bufferIndex],
                  bufferBytes );
      }

      if ( stream_.doByteSwap[1] )
        byteSwapBuffer( stream_.deviceBuffer,
                        stream_.bufferSize * stream_.nDeviceChannels[1],
                        stream_.deviceFormat[1] );
      convertBuffer( stream_.userBuffer[1], stream_.deviceBuffer, stream_.convertInfo[1] );

    }
    else {
      for ( i=0, j=0; i<nChannels; i++ ) {
        if ( handle->bufferInfos[i].isInput == ASIOTrue ) {
          memcpy( &stream_.userBuffer[1][bufferBytes*j++],
                  handle->bufferInfos[i].buffers[bufferIndex],
                  bufferBytes );
        }
      }

      if ( stream_.doByteSwap[1] )
        byteSwapBuffer( stream_.userBuffer[1],
                        stream_.bufferSize * stream_.nUserChannels[1],
                        stream_.userFormat );
    }
  }

 unlock:
  // The following call was suggested by Malte Clasen.  While the API
  // documentation indicates it should not be required, some device
  // drivers apparently do not function correctly without it.
  ASIOOutputReady();

  RtApi::tickStreamTime();
  return SUCCESS;
}

static void sampleRateChanged( ASIOSampleRate sRate )
{
  // The ASIO documentation says that this usually only happens during
  // external sync.  Audio processing is not stopped by the driver,
  // actual sample rate might not have even changed, maybe only the
  // sample rate status of an AES/EBU or S/PDIF digital input at the
  // audio device.

  RtApi *object = (RtApi *) asioCallbackInfo->object;
  try {
    object->stopStream();
  }
  catch ( RtAudioError &exception ) {
    std::cerr << "\nRtApiAsio: sampleRateChanged() error (" << exception.getMessage() << ")!\n" << std::endl;
    return;
  }

  std::cerr << "\nRtApiAsio: driver reports sample rate changed to " << sRate << " ... stream stopped!!!\n" << std::endl;
}

static long asioMessages( long selector, long value, void* /*message*/, double* /*opt*/ )
{
  long ret = 0;

  switch( selector ) {
  case kAsioSelectorSupported:
    if ( value == kAsioResetRequest
         || value == kAsioEngineVersion
         || value == kAsioResyncRequest
         || value == kAsioLatenciesChanged
         // The following three were added for ASIO 2.0, you don't
         // necessarily have to support them.
         || value == kAsioSupportsTimeInfo
         || value == kAsioSupportsTimeCode
         || value == kAsioSupportsInputMonitor)
      ret = 1L;
    break;
  case kAsioResetRequest:
    // Defer the task and perform the reset of the driver during the
    // next "safe" situation.  You cannot reset the driver right now,
    // as this code is called from the driver.  Reset the driver is
    // done by completely destruct is. I.e. ASIOStop(),
    // ASIODisposeBuffers(), Destruction Afterwards you initialize the
    // driver again.
    std::cerr << "\nRtApiAsio: driver reset requested!!!" << std::endl;
    ret = 1L;
    break;
  case kAsioResyncRequest:
    // This informs the application that the driver encountered some
    // non-fatal data loss.  It is used for synchronization purposes
    // of different media.  Added mainly to work around the Win16Mutex
    // problems in Windows 95/98 with the Windows Multimedia system,
    // which could lose data because the Mutex was held too long by
    // another thread.  However a driver can issue it in other
    // situations, too.
    // std::cerr << "\nRtApiAsio: driver resync requested!!!" << std::endl;
    asioXRun = true;
    ret = 1L;
    break;
  case kAsioLatenciesChanged:
    // This will inform the host application that the drivers were
    // latencies changed.  Beware, it this does not mean that the
    // buffer sizes have changed!  You might need to update internal
    // delay data.
    std::cerr << "\nRtApiAsio: driver latency may have changed!!!" << std::endl;
    ret = 1L;
    break;
  case kAsioEngineVersion:
    // Return the supported ASIO version of the host application.  If
    // a host application does not implement this selector, ASIO 1.0
    // is assumed by the driver.
    ret = 2L;
    break;
  case kAsioSupportsTimeInfo:
    // Informs the driver whether the
    // asioCallbacks.bufferSwitchTimeInfo() callback is supported.
    // For compatibility with ASIO 1.0 drivers the host application
    // should always support the "old" bufferSwitch method, too.
    ret = 0;
    break;
  case kAsioSupportsTimeCode:
    // Informs the driver whether application is interested in time
    // code info.  If an application does not need to know about time
    // code, the driver has less work to do.
    ret = 0;
    break;
  }
  return ret;
}

static const char* getAsioErrorString( ASIOError result )
{
  struct Messages 
  {
    ASIOError value;
    const char*message;
  };

  static const Messages m[] = 
    {
      {   ASE_NotPresent,    "Hardware input or output is not present or available." },
      {   ASE_HWMalfunction,  "Hardware is malfunctioning." },
      {   ASE_InvalidParameter, "Invalid input parameter." },
      {   ASE_InvalidMode,      "Invalid mode." },
      {   ASE_SPNotAdvancing,     "Sample position not advancing." },
      {   ASE_NoClock,            "Sample clock or rate cannot be determined or is not present." },
      {   ASE_NoMemory,           "Not enough memory to complete the request." }
    };

  for ( unsigned int i = 0; i < sizeof(m)/sizeof(m[0]); ++i )
    if ( m[i].value == result ) return m[i].message;

  return "Unknown error.";
}

//******************** End of __WINDOWS_ASIO__ *********************//
#endif


#if defined(__WINDOWS_WASAPI__) // Windows WASAPI API

// Authored by Marcus Tomlinson <themarcustomlinson@gmail.com>, April 2014
// - Introduces support for the Windows WASAPI API
// - Aims to deliver bit streams to and from hardware at the lowest possible latency, via the absolute minimum buffer sizes required
// - Provides flexible stream configuration to an otherwise strict and inflexible WASAPI interface
// - Includes automatic internal conversion of sample rate and buffer size between hardware and the user

#ifndef INITGUID
  #define INITGUID
#endif
#include <audioclient.h>
#include <avrt.h>
#include <mmdeviceapi.h>

#if PLATFORM_WINDOWS
#include <functiondiscoverykeys_devpkey.h>
#endif

#include <sstream>

//=============================================================================

#if !defined(SAFE_RELEASE)
	#define SAFE_RELEASE( objectPtr )\
	if ( objectPtr )\
	{\
	  objectPtr->Release();\
	  objectPtr = NULL;\
	}
#endif

typedef HANDLE ( __stdcall *TAvSetMmThreadCharacteristicsPtr )( LPCWSTR TaskName, LPDWORD TaskIndex );

//-----------------------------------------------------------------------------

// WASAPI dictates stream sample rate, format, channel count, and in some cases, buffer size.
// Therefore we must perform all necessary conversions to user buffers in order to satisfy these
// requirements. WasapiBuffer ring buffers are used between HwIn->UserIn and UserOut->HwOut to
// provide intermediate storage for read / write synchronization.
class WasapiBuffer
{
public:
  WasapiBuffer()
    : buffer_( NULL ),
      bufferSize_( 0 ),
      inIndex_( 0 ),
      outIndex_( 0 ) {}

  ~WasapiBuffer() {
    free( buffer_ );
  }

  // sets the length of the internal ring buffer
  void setBufferSize( unsigned int bufferSize, unsigned int formatBytes ) {
    free( buffer_ );

    buffer_ = ( char* ) calloc( bufferSize, formatBytes );

    bufferSize_ = bufferSize;
    inIndex_ = 0;
    outIndex_ = 0;
  }

  // attempt to push a buffer into the ring buffer at the current "in" index
  bool pushBuffer( char* buffer, unsigned int bufferSize, RtAudioFormat format )
  {
    if ( !buffer ||                 // incoming buffer is NULL
         bufferSize == 0 ||         // incoming buffer has no data
         bufferSize > bufferSize_ ) // incoming buffer too large
    {
      return false;
    }

    unsigned int relOutIndex = outIndex_;
    unsigned int inIndexEnd = inIndex_ + bufferSize;
    if ( relOutIndex < inIndex_ && inIndexEnd >= bufferSize_ ) {
      relOutIndex += bufferSize_;
    }

    // "in" index can end on the "out" index but cannot begin at it
    if ( inIndex_ <= relOutIndex && inIndexEnd > relOutIndex ) {
      return false; // not enough space between "in" index and "out" index
    }

    // copy buffer from external to internal
    int fromZeroSize = inIndex_ + bufferSize - bufferSize_;
    fromZeroSize = fromZeroSize < 0 ? 0 : fromZeroSize;
    int fromInSize = bufferSize - fromZeroSize;

    switch( format )
      {
      case RTAUDIO_SINT8:
        memcpy( &( ( char* ) buffer_ )[inIndex_], buffer, fromInSize * sizeof( char ) );
        memcpy( buffer_, &( ( char* ) buffer )[fromInSize], fromZeroSize * sizeof( char ) );
        break;
      case RTAUDIO_SINT16:
        memcpy( &( ( short* ) buffer_ )[inIndex_], buffer, fromInSize * sizeof( short ) );
        memcpy( buffer_, &( ( short* ) buffer )[fromInSize], fromZeroSize * sizeof( short ) );
        break;
      case RTAUDIO_SINT24:
        memcpy( &( ( S24* ) buffer_ )[inIndex_], buffer, fromInSize * sizeof( S24 ) );
        memcpy( buffer_, &( ( S24* ) buffer )[fromInSize], fromZeroSize * sizeof( S24 ) );
        break;
      case RTAUDIO_SINT32:
        memcpy( &( ( int* ) buffer_ )[inIndex_], buffer, fromInSize * sizeof( int ) );
        memcpy( buffer_, &( ( int* ) buffer )[fromInSize], fromZeroSize * sizeof( int ) );
        break;
      case RTAUDIO_FLOAT32:
        memcpy( &( ( float* ) buffer_ )[inIndex_], buffer, fromInSize * sizeof( float ) );
        memcpy( buffer_, &( ( float* ) buffer )[fromInSize], fromZeroSize * sizeof( float ) );
        break;
      case RTAUDIO_FLOAT64:
        memcpy( &( ( double* ) buffer_ )[inIndex_], buffer, fromInSize * sizeof( double ) );
        memcpy( buffer_, &( ( double* ) buffer )[fromInSize], fromZeroSize * sizeof( double ) );
        break;
    }

    // update "in" index
    inIndex_ += bufferSize;
    inIndex_ %= bufferSize_;

    return true;
  }

  // attempt to pull a buffer from the ring buffer from the current "out" index
  bool pullBuffer( char* buffer, unsigned int bufferSize, RtAudioFormat format )
  {
    if ( !buffer ||                 // incoming buffer is NULL
         bufferSize == 0 ||         // incoming buffer has no data
         bufferSize > bufferSize_ ) // incoming buffer too large
    {
      return false;
    }

    unsigned int relInIndex = inIndex_;
    unsigned int outIndexEnd = outIndex_ + bufferSize;
    if ( relInIndex < outIndex_ && outIndexEnd >= bufferSize_ ) {
      relInIndex += bufferSize_;
    }

    // "out" index can begin at and end on the "in" index
    if ( outIndex_ < relInIndex && outIndexEnd > relInIndex ) {
      return false; // not enough space between "out" index and "in" index
    }

    // copy buffer from internal to external
    int fromZeroSize = outIndex_ + bufferSize - bufferSize_;
    fromZeroSize = fromZeroSize < 0 ? 0 : fromZeroSize;
    int fromOutSize = bufferSize - fromZeroSize;

    switch( format )
    {
      case RTAUDIO_SINT8:
        memcpy( buffer, &( ( char* ) buffer_ )[outIndex_], fromOutSize * sizeof( char ) );
        memcpy( &( ( char* ) buffer )[fromOutSize], buffer_, fromZeroSize * sizeof( char ) );
        break;
      case RTAUDIO_SINT16:
        memcpy( buffer, &( ( short* ) buffer_ )[outIndex_], fromOutSize * sizeof( short ) );
        memcpy( &( ( short* ) buffer )[fromOutSize], buffer_, fromZeroSize * sizeof( short ) );
        break;
      case RTAUDIO_SINT24:
        memcpy( buffer, &( ( S24* ) buffer_ )[outIndex_], fromOutSize * sizeof( S24 ) );
        memcpy( &( ( S24* ) buffer )[fromOutSize], buffer_, fromZeroSize * sizeof( S24 ) );
        break;
      case RTAUDIO_SINT32:
        memcpy( buffer, &( ( int* ) buffer_ )[outIndex_], fromOutSize * sizeof( int ) );
        memcpy( &( ( int* ) buffer )[fromOutSize], buffer_, fromZeroSize * sizeof( int ) );
        break;
      case RTAUDIO_FLOAT32:
        memcpy( buffer, &( ( float* ) buffer_ )[outIndex_], fromOutSize * sizeof( float ) );
        memcpy( &( ( float* ) buffer )[fromOutSize], buffer_, fromZeroSize * sizeof( float ) );
        break;
      case RTAUDIO_FLOAT64:
        memcpy( buffer, &( ( double* ) buffer_ )[outIndex_], fromOutSize * sizeof( double ) );
        memcpy( &( ( double* ) buffer )[fromOutSize], buffer_, fromZeroSize * sizeof( double ) );
        break;
    }

    // update "out" index
    outIndex_ += bufferSize;
    outIndex_ %= bufferSize_;

    return true;
  }

private:
  char* buffer_;
  unsigned int bufferSize_;
  unsigned int inIndex_;
  unsigned int outIndex_;
};

//-----------------------------------------------------------------------------

// A structure to hold various information related to the WASAPI implementation.
struct WasapiHandle
{
  IAudioClient* captureAudioClient;
  IAudioClient* renderAudioClient;
  IAudioCaptureClient* captureClient;
  IAudioRenderClient* renderClient;
  HANDLE captureEvent;
  HANDLE renderEvent;

  WasapiHandle()
  : captureAudioClient( NULL ),
    renderAudioClient( NULL ),
    captureClient( NULL ),
    renderClient( NULL ),
    captureEvent( NULL ),
    renderEvent( NULL ) {}
};

//=============================================================================

RtApiWasapi::RtApiWasapi()
  : coInitialized_( false ), deviceEnumerator_( NULL )
{
#if PLATFORM_WINDOWS
  // WASAPI can run either apartment or multi-threaded
  HRESULT hr = CoInitialize( NULL );
  if ( !FAILED( hr ) )
    coInitialized_ = true;
#else
  HRESULT hr;
#endif // PLATFORM_WINDOWS

  // Instantiate device enumerator
  hr = CoCreateInstance( __uuidof( MMDeviceEnumerator ), NULL,
                         CLSCTX_ALL, __uuidof( IMMDeviceEnumerator ),
                         ( void** ) &deviceEnumerator_ );

  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::RtApiWasapi: Unable to instantiate device enumerator";
    error( RtAudioError::DRIVER_ERROR );
  }
}

//-----------------------------------------------------------------------------

RtApiWasapi::~RtApiWasapi()
{
  if ( stream_.state != STREAM_CLOSED )
    closeStream();

  SAFE_RELEASE( deviceEnumerator_ );

  // If this object previously called CoInitialize()
  if ( coInitialized_ )
    CoUninitialize();
}

//=============================================================================

unsigned int RtApiWasapi::getDeviceCount( void )
{
  unsigned int captureDeviceCount = 0;
  unsigned int renderDeviceCount = 0;

  IMMDeviceCollection* captureDevices = NULL;
  IMMDeviceCollection* renderDevices = NULL;

  // Count capture devices
  errorText_.clear();
  HRESULT hr = deviceEnumerator_->EnumAudioEndpoints( eCapture, DEVICE_STATE_ACTIVE, &captureDevices );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceCount: Unable to retrieve capture device collection.";
    goto Exit;
  }

  hr = captureDevices->GetCount( &captureDeviceCount );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceCount: Unable to retrieve capture device count.";
    goto Exit;
  }

  // Count render devices
  hr = deviceEnumerator_->EnumAudioEndpoints( eRender, DEVICE_STATE_ACTIVE, &renderDevices );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceCount: Unable to retrieve render device collection.";
    goto Exit;
  }

  hr = renderDevices->GetCount( &renderDeviceCount );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceCount: Unable to retrieve render device count.";
    goto Exit;
  }

Exit:
  // release all references
  SAFE_RELEASE( captureDevices );
  SAFE_RELEASE( renderDevices );

  if ( errorText_.empty() )
    return captureDeviceCount + renderDeviceCount;

  error( RtAudioError::DRIVER_ERROR );
  return 0;
}

//-----------------------------------------------------------------------------

RtAudio::DeviceInfo RtApiWasapi::getDeviceInfo( unsigned int device )
{
  RtAudio::DeviceInfo info;
  unsigned int captureDeviceCount = 0;
  unsigned int renderDeviceCount = 0;
  bool isCaptureDevice = false;
  std::string defaultDeviceId;
  LPWSTR strDefaultDeviceId = NULL;
  LPWSTR strDeviceId = NULL;

  PROPVARIANT deviceNameProp = {};

  IMMDeviceCollection* captureDevices = NULL;
  IMMDeviceCollection* renderDevices = NULL;
  IMMDevice* devicePtr = NULL;
  IMMDevice* defaultDevicePtr = NULL;
  IAudioClient* audioClient = NULL;
  IPropertyStore* devicePropStore = NULL;
  IPropertyStore* defaultDevicePropStore = NULL;

  WAVEFORMATEX* deviceFormat = NULL;

  // probed
  info.probed = false;

  // Count capture devices
  errorText_.clear();
  RtAudioError::Type errorType = RtAudioError::DRIVER_ERROR;
  HRESULT hr = deviceEnumerator_->EnumAudioEndpoints( eCapture, DEVICE_STATE_ACTIVE, &captureDevices );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve capture device collection.";
    goto Exit;
  }

  hr = captureDevices->GetCount( &captureDeviceCount );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve capture device count.";
    goto Exit;
  }

  // Count render devices
  hr = deviceEnumerator_->EnumAudioEndpoints( eRender, DEVICE_STATE_ACTIVE, &renderDevices );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve render device collection.";
    goto Exit;
  }

  hr = renderDevices->GetCount( &renderDeviceCount );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve render device count.";
    goto Exit;
  }

  // validate device index
  if ( device >= captureDeviceCount + renderDeviceCount ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Invalid device index.";
    errorType = RtAudioError::INVALID_USE;
    goto Exit;
  }

  // determine whether index falls within capture or render devices
  if ( device >= renderDeviceCount ) {
    hr = captureDevices->Item( device - renderDeviceCount, &devicePtr );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve capture device handle.";
      goto Exit;
    }
    isCaptureDevice = true;
  }
  else {
    hr = renderDevices->Item( device, &devicePtr );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve render device handle.";
      goto Exit;
    }
    isCaptureDevice = false;
  }

  // get default device name
  if ( isCaptureDevice ) {
    hr = deviceEnumerator_->GetDefaultAudioEndpoint( eCapture, eConsole, &defaultDevicePtr );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve default capture device handle.";
      goto Exit;
    }
  }
  else {
    hr = deviceEnumerator_->GetDefaultAudioEndpoint( eRender, eConsole, &defaultDevicePtr );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve default render device handle.";
      goto Exit;
    }
  }

  hr = defaultDevicePtr->GetId(&strDefaultDeviceId);
  if (FAILED(hr)) {
	  errorText_ = "RtApiWasapi::getDeviceInfo: Unable to get default device Id.";
	  goto Exit;
  }
  defaultDeviceId = convertCharPointerToStdString(strDefaultDeviceId);

  hr = devicePtr->GetId(&strDeviceId);
  if (FAILED(hr)) {
	  errorText_ = "RtApiWasapi::getDeviceInfo: Unable to get device Id.";
	  goto Exit;
  }
  info.deviceId = convertCharPointerToStdString(strDeviceId);

#if PLATFORM_WINDOWS
  hr = defaultDevicePtr->OpenPropertyStore( STGM_READ, &defaultDevicePropStore );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to open default device property store.";
    goto Exit;
  }
 
  // name
  hr = devicePtr->OpenPropertyStore( STGM_READ, &devicePropStore );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to open device property store.";
    goto Exit;
  }

  PropVariantInit( &deviceNameProp );

  hr = devicePropStore->GetValue( PKEY_Device_FriendlyName, &deviceNameProp );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve device property: PKEY_Device_FriendlyName.";
    goto Exit;
  }

  info.name = convertCharPointerToStdString(deviceNameProp.pwszVal);
#else // On Xbox One, we hardcode out defaults.
  info.name = std::string("Xbox Audio Input ");
#endif

  // is default
  if ( isCaptureDevice ) {
    info.isDefaultInput = info.deviceId == defaultDeviceId;
    info.isDefaultOutput = false;
  }
  else {
    info.isDefaultInput = false;
    info.isDefaultOutput = info.deviceId == defaultDeviceId;;
  }

  // channel count
  hr = devicePtr->Activate( __uuidof( IAudioClient ), CLSCTX_ALL, NULL, ( void** ) &audioClient );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve device audio client.";
    goto Exit;
  }

  hr = audioClient->GetMixFormat( &deviceFormat );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::getDeviceInfo: Unable to retrieve device mix format.";
    goto Exit;
  }

  if ( isCaptureDevice ) {
    info.inputChannels = deviceFormat->nChannels;
    info.outputChannels = 0;
    info.duplexChannels = 0;
  }
  else {
    info.inputChannels = 0;
    info.outputChannels = deviceFormat->nChannels;
    info.duplexChannels = 0;
  }

  // sample rates (WASAPI only supports the one native sample rate)
  info.preferredSampleRate = deviceFormat->nSamplesPerSec;

  info.sampleRates.clear();
  info.sampleRates.push_back( deviceFormat->nSamplesPerSec );

  // native format
  info.nativeFormats = 0;

  if ( deviceFormat->wFormatTag == WAVE_FORMAT_IEEE_FLOAT ||
       ( deviceFormat->wFormatTag == WAVE_FORMAT_EXTENSIBLE &&
         ( ( WAVEFORMATEXTENSIBLE* ) deviceFormat )->SubFormat == KSDATAFORMAT_SUBTYPE_IEEE_FLOAT ) )
  {
    if ( deviceFormat->wBitsPerSample == 32 ) {
      info.nativeFormats |= RTAUDIO_FLOAT32;
    }
    else if ( deviceFormat->wBitsPerSample == 64 ) {
      info.nativeFormats |= RTAUDIO_FLOAT64;
    }
  }
  else if ( deviceFormat->wFormatTag == WAVE_FORMAT_PCM ||
           ( deviceFormat->wFormatTag == WAVE_FORMAT_EXTENSIBLE &&
             ( ( WAVEFORMATEXTENSIBLE* ) deviceFormat )->SubFormat == KSDATAFORMAT_SUBTYPE_PCM ) )
  {
    if ( deviceFormat->wBitsPerSample == 8 ) {
      info.nativeFormats |= RTAUDIO_SINT8;
    }
    else if ( deviceFormat->wBitsPerSample == 16 ) {
      info.nativeFormats |= RTAUDIO_SINT16;
    }
    else if ( deviceFormat->wBitsPerSample == 24 ) {
      info.nativeFormats |= RTAUDIO_SINT24;
    }
    else if ( deviceFormat->wBitsPerSample == 32 ) {
      info.nativeFormats |= RTAUDIO_SINT32;
    }
  }

  // probed
  info.probed = true;

Exit:
  // release all references
  PropVariantClear( &deviceNameProp );

  SAFE_RELEASE( captureDevices );
  SAFE_RELEASE( renderDevices );
  SAFE_RELEASE( devicePtr );
  SAFE_RELEASE( defaultDevicePtr );
  SAFE_RELEASE( audioClient );
  SAFE_RELEASE( devicePropStore );
  SAFE_RELEASE( defaultDevicePropStore );

  CoTaskMemFree( deviceFormat );

  if ( !errorText_.empty() )
    error( errorType );
  return info;
}

//-----------------------------------------------------------------------------

unsigned int RtApiWasapi::getDefaultOutputDevice( void )
{
  for ( unsigned int i = 0; i < getDeviceCount(); i++ ) {
    if ( getDeviceInfo( i ).isDefaultOutput ) {
      return i;
    }
  }

  return 0;
}

//-----------------------------------------------------------------------------

unsigned int RtApiWasapi::getDefaultInputDevice( void )
{
  for ( unsigned int i = 0; i < getDeviceCount(); i++ ) {
    if ( getDeviceInfo( i ).isDefaultInput ) {
      return i;
    }
  }

  return 0;
}

//-----------------------------------------------------------------------------

void RtApiWasapi::closeStream( void )
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiWasapi::closeStream: No open stream to close.";
    error( RtAudioError::WARNING );
    return;
  }

  if ( stream_.state != STREAM_STOPPED )
    stopStream();

  // clean up stream memory
  SAFE_RELEASE( ( ( WasapiHandle* ) stream_.apiHandle )->captureAudioClient )
  SAFE_RELEASE( ( ( WasapiHandle* ) stream_.apiHandle )->renderAudioClient )

  SAFE_RELEASE( ( ( WasapiHandle* ) stream_.apiHandle )->captureClient )
  SAFE_RELEASE( ( ( WasapiHandle* ) stream_.apiHandle )->renderClient )

  if ( ( ( WasapiHandle* ) stream_.apiHandle )->captureEvent )
    CloseHandle( ( ( WasapiHandle* ) stream_.apiHandle )->captureEvent );

  if ( ( ( WasapiHandle* ) stream_.apiHandle )->renderEvent )
    CloseHandle( ( ( WasapiHandle* ) stream_.apiHandle )->renderEvent );

  delete ( WasapiHandle* ) stream_.apiHandle;
  stream_.apiHandle = NULL;

  for ( int i = 0; i < 2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  // update stream state
  stream_.state = STREAM_CLOSED;
}

//-----------------------------------------------------------------------------

void RtApiWasapi::startStream( void )
{
  verifyStream();

  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiWasapi::startStream: The stream is already running.";
    error( RtAudioError::WARNING );
    return;
  }

  // update stream state
  stream_.state = STREAM_RUNNING;

  // create WASAPI stream thread
  stream_.callbackInfo.thread = ( ThreadHandle ) CreateThread( NULL, 0, runWasapiThread, this, CREATE_SUSPENDED, NULL );

  if ( !stream_.callbackInfo.thread ) {
    errorText_ = "RtApiWasapi::startStream: Unable to instantiate callback thread.";
    error( RtAudioError::THREAD_ERROR );
  }
  else {
    SetThreadPriority( ( void* ) stream_.callbackInfo.thread, stream_.callbackInfo.priority );
    ResumeThread( ( void* ) stream_.callbackInfo.thread );
  }
}

//-----------------------------------------------------------------------------

void RtApiWasapi::stopStream( void )
{
  verifyStream();

  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiWasapi::stopStream: The stream is already stopped.";
    error( RtAudioError::WARNING );
    return;
  }

  // inform stream thread by setting stream state to STREAM_STOPPING
  stream_.state = STREAM_STOPPING;

  // wait until stream thread is stopped
  while( stream_.state != STREAM_STOPPED ) {
    Sleep( 1 );
  }

  // Wait for the last buffer to play before stopping.
  Sleep( 1000 * stream_.bufferSize / stream_.sampleRate );

  // stop capture client if applicable
  if ( ( ( WasapiHandle* ) stream_.apiHandle )->captureAudioClient ) {
    HRESULT hr = ( ( WasapiHandle* ) stream_.apiHandle )->captureAudioClient->Stop();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::stopStream: Unable to stop capture stream.";
      error( RtAudioError::DRIVER_ERROR );
      return;
    }
  }

  // stop render client if applicable
  if ( ( ( WasapiHandle* ) stream_.apiHandle )->renderAudioClient ) {
    HRESULT hr = ( ( WasapiHandle* ) stream_.apiHandle )->renderAudioClient->Stop();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::stopStream: Unable to stop render stream.";
      error( RtAudioError::DRIVER_ERROR );
      return;
    }
  }

  // close thread handle
  if ( stream_.callbackInfo.thread && !CloseHandle( ( void* ) stream_.callbackInfo.thread ) ) {
    errorText_ = "RtApiWasapi::stopStream: Unable to close callback thread.";
    error( RtAudioError::THREAD_ERROR );
    return;
  }

  stream_.callbackInfo.thread = (ThreadHandle) NULL;
}

//-----------------------------------------------------------------------------

void RtApiWasapi::abortStream( void )
{
  verifyStream();

  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiWasapi::abortStream: The stream is already stopped.";
    error( RtAudioError::WARNING );
    return;
  }

  // inform stream thread by setting stream state to STREAM_STOPPING
  stream_.state = STREAM_STOPPING;

  // wait until stream thread is stopped
  while ( stream_.state != STREAM_STOPPED ) {
    Sleep( 1 );
  }

  // stop capture client if applicable
  if ( ( ( WasapiHandle* ) stream_.apiHandle )->captureAudioClient ) {
    HRESULT hr = ( ( WasapiHandle* ) stream_.apiHandle )->captureAudioClient->Stop();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::abortStream: Unable to stop capture stream.";
      error( RtAudioError::DRIVER_ERROR );
      return;
    }
  }

  // stop render client if applicable
  if ( ( ( WasapiHandle* ) stream_.apiHandle )->renderAudioClient ) {
    HRESULT hr = ( ( WasapiHandle* ) stream_.apiHandle )->renderAudioClient->Stop();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::abortStream: Unable to stop render stream.";
      error( RtAudioError::DRIVER_ERROR );
      return;
    }
  }

  // close thread handle
  if ( stream_.callbackInfo.thread && !CloseHandle( ( void* ) stream_.callbackInfo.thread ) ) {
    errorText_ = "RtApiWasapi::abortStream: Unable to close callback thread.";
    error( RtAudioError::THREAD_ERROR );
    return;
  }

  stream_.callbackInfo.thread = (ThreadHandle) NULL;
}

//-----------------------------------------------------------------------------

bool RtApiWasapi::probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                                   unsigned int firstChannel, unsigned int sampleRate,
                                   RtAudioFormat format, unsigned int* bufferSize,
                                   RtAudio::StreamOptions* options )
{
  bool methodResult = FAILURE;
  unsigned int captureDeviceCount = 0;
  unsigned int renderDeviceCount = 0;

  IMMDeviceCollection* captureDevices = NULL;
  IMMDeviceCollection* renderDevices = NULL;
  IMMDevice* devicePtr = NULL;
  WAVEFORMATEX* deviceFormat = NULL;
  unsigned int bufferBytes;
  stream_.state = STREAM_STOPPED;
  RtAudio::DeviceInfo deviceInfo;

  // create API Handle if not already created
  if ( !stream_.apiHandle )
    stream_.apiHandle = ( void* ) new WasapiHandle();

  // Count capture devices
  errorText_.clear();
  RtAudioError::Type errorType = RtAudioError::DRIVER_ERROR;
  HRESULT hr = deviceEnumerator_->EnumAudioEndpoints( eCapture, DEVICE_STATE_ACTIVE, &captureDevices );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve capture device collection.";
    goto Exit;
  }

  hr = captureDevices->GetCount( &captureDeviceCount );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve capture device count.";
    goto Exit;
  }

  // Count render devices
  hr = deviceEnumerator_->EnumAudioEndpoints( eRender, DEVICE_STATE_ACTIVE, &renderDevices );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve render device collection.";
    goto Exit;
  }

  hr = renderDevices->GetCount( &renderDeviceCount );
  if ( FAILED( hr ) ) {
    errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve render device count.";
    goto Exit;
  }

  // validate device index
  if ( device >= captureDeviceCount + renderDeviceCount ) {
    errorType = RtAudioError::INVALID_USE;
    errorText_ = "RtApiWasapi::probeDeviceOpen: Invalid device index.";
    goto Exit;
  }

  deviceInfo = getDeviceInfo( device );

  // validate sample rate
  if ( sampleRate != deviceInfo.preferredSampleRate )
  {
    errorType = RtAudioError::INVALID_USE;
    std::stringstream ss;
    ss << "RtApiWasapi::probeDeviceOpen: " << sampleRate
       << "Hz sample rate not supported. This device only supports "
       << deviceInfo.preferredSampleRate << "Hz.";
    errorText_ = ss.str();
    goto Exit;
  }

  // determine whether index falls within capture or render devices
  if ( device >= renderDeviceCount ) {
    if ( mode != INPUT ) {
      errorType = RtAudioError::INVALID_USE;
      errorText_ = "RtApiWasapi::probeDeviceOpen: Capture device selected as output device.";
      goto Exit;
    }

    // retrieve captureAudioClient from devicePtr
    IAudioClient*& captureAudioClient = ( ( WasapiHandle* ) stream_.apiHandle )->captureAudioClient;

    hr = captureDevices->Item( device - renderDeviceCount, &devicePtr );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve capture device handle.";
      goto Exit;
    }

    hr = devicePtr->Activate( __uuidof( IAudioClient ), CLSCTX_ALL,
                              NULL, ( void** ) &captureAudioClient );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve device audio client.";
      goto Exit;
    }

    hr = captureAudioClient->GetMixFormat( &deviceFormat );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve device mix format.";
      goto Exit;
    }

    stream_.nDeviceChannels[mode] = deviceFormat->nChannels;
    captureAudioClient->GetStreamLatency( ( long long* ) &stream_.latency[mode] );
  }
  else {
    if ( mode != OUTPUT ) {
      errorType = RtAudioError::INVALID_USE;
      errorText_ = "RtApiWasapi::probeDeviceOpen: Render device selected as input device.";
      goto Exit;
    }

    // retrieve renderAudioClient from devicePtr
    IAudioClient*& renderAudioClient = ( ( WasapiHandle* ) stream_.apiHandle )->renderAudioClient;

    hr = renderDevices->Item( device, &devicePtr );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve render device handle.";
      goto Exit;
    }

    hr = devicePtr->Activate( __uuidof( IAudioClient ), CLSCTX_ALL,
                              NULL, ( void** ) &renderAudioClient );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve device audio client.";
      goto Exit;
    }

    hr = renderAudioClient->GetMixFormat( &deviceFormat );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::probeDeviceOpen: Unable to retrieve device mix format.";
      goto Exit;
    }

    stream_.nDeviceChannels[mode] = deviceFormat->nChannels;
    renderAudioClient->GetStreamLatency( ( long long* ) &stream_.latency[mode] );
  }

  // fill stream data
  if ( ( stream_.mode == OUTPUT && mode == INPUT ) ||
       ( stream_.mode == INPUT && mode == OUTPUT ) ) {
    stream_.mode = DUPLEX;
  }
  else {
    stream_.mode = mode;
  }

  stream_.device[mode] = device;
  stream_.doByteSwap[mode] = false;
  stream_.sampleRate = sampleRate;
  stream_.bufferSize = *bufferSize;
  stream_.nBuffers = 1;
  stream_.nUserChannels[mode] = channels;
  stream_.channelOffset[mode] = firstChannel;
  stream_.userFormat = format;
  stream_.deviceFormat[mode] = deviceInfo.nativeFormats;

  if ( options && options->flags & RTAUDIO_NONINTERLEAVED )
    stream_.userInterleaved = false;
  else
    stream_.userInterleaved = true;
  stream_.deviceInterleaved[mode] = true;

  // Set flags for buffer conversion.
  stream_.doConvertBuffer[mode] = false;
  if ( stream_.userFormat != stream_.deviceFormat[mode] ||
       stream_.nUserChannels[0] != stream_.nDeviceChannels[0] ||
	   stream_.nUserChannels[1] != stream_.nDeviceChannels[1])
    stream_.doConvertBuffer[mode] = true;
  else if ( stream_.userInterleaved != stream_.deviceInterleaved[mode] &&
            stream_.nUserChannels[mode] > 1 )
    stream_.doConvertBuffer[mode] = true;

  if ( stream_.doConvertBuffer[mode] )
    setConvertInfo( mode, 0 );

  // Allocate necessary internal buffers
  bufferBytes = stream_.nUserChannels[mode] * stream_.bufferSize * formatBytes( stream_.userFormat );

  stream_.userBuffer[mode] = ( char* ) calloc( bufferBytes, 1 );
  if ( !stream_.userBuffer[mode] ) {
    errorType = RtAudioError::MEMORY_ERROR;
    errorText_ = "RtApiWasapi::probeDeviceOpen: Error allocating user buffer memory.";
    goto Exit;
  }

  if ( options && options->flags & RTAUDIO_SCHEDULE_REALTIME )
    stream_.callbackInfo.priority = 15;
  else
    stream_.callbackInfo.priority = 0;

  ///! TODO: RTAUDIO_MINIMIZE_LATENCY // Provide stream buffers directly to callback
  ///! TODO: RTAUDIO_HOG_DEVICE       // Exclusive mode

  methodResult = SUCCESS;

Exit:
  //clean up
  SAFE_RELEASE( captureDevices );
  SAFE_RELEASE( renderDevices );
  SAFE_RELEASE( devicePtr );
  CoTaskMemFree( deviceFormat );

  // if method failed, close the stream
  if ( methodResult == FAILURE )
    closeStream();

  if ( !errorText_.empty() )
    error( errorType );
  return methodResult;
}

//=============================================================================

DWORD WINAPI RtApiWasapi::runWasapiThread( void* wasapiPtr )
{
  if ( wasapiPtr )
    ( ( RtApiWasapi* ) wasapiPtr )->wasapiThread();

  return 0;
}

DWORD WINAPI RtApiWasapi::stopWasapiThread( void* wasapiPtr )
{
  if ( wasapiPtr )
    ( ( RtApiWasapi* ) wasapiPtr )->stopStream();

  return 0;
}

DWORD WINAPI RtApiWasapi::abortWasapiThread( void* wasapiPtr )
{
  if ( wasapiPtr )
    ( ( RtApiWasapi* ) wasapiPtr )->abortStream();

  return 0;
}

//-----------------------------------------------------------------------------

void RtApiWasapi::wasapiThread()
{
#if PLATFORM_WINDOWS
  // as this is a new thread, we must CoInitialize it
  CoInitialize( NULL );
#endif // PLATFORM_WINDOWS

  HRESULT hr;

  IAudioClient* captureAudioClient = ( ( WasapiHandle* ) stream_.apiHandle )->captureAudioClient;
  IAudioClient* renderAudioClient = ( ( WasapiHandle* ) stream_.apiHandle )->renderAudioClient;
  IAudioCaptureClient* captureClient = ( ( WasapiHandle* ) stream_.apiHandle )->captureClient;
  IAudioRenderClient* renderClient = ( ( WasapiHandle* ) stream_.apiHandle )->renderClient;
  HANDLE captureEvent = ( ( WasapiHandle* ) stream_.apiHandle )->captureEvent;
  HANDLE renderEvent = ( ( WasapiHandle* ) stream_.apiHandle )->renderEvent;

  WAVEFORMATEX* captureFormat = NULL;
  WAVEFORMATEX* renderFormat = NULL;
  WasapiBuffer captureBuffer;
  WasapiBuffer renderBuffer;

  // declare local stream variables
  RtAudioCallback callback = ( RtAudioCallback ) stream_.callbackInfo.callback;
  BYTE* streamBuffer = NULL;
  unsigned long captureFlags = 0;
  unsigned int bufferFrameCount = 0;
  unsigned int numFramesPadding = 0;
  bool callbackPushed = false;
  bool callbackPulled = false;
  bool callbackStopped = false;
  int callbackResult = 0;

  unsigned int deviceBuffSize = 0;

  errorText_.clear();
  RtAudioError::Type errorType = RtAudioError::DRIVER_ERROR;

#if PLATFORM_WINDOWS
  // Attempt to assign "Pro Audio" characteristic to thread
  HMODULE AvrtDll = LoadLibrary( (LPCTSTR) "AVRT.dll" );
  if ( AvrtDll ) {
    DWORD taskIndex = 0;
    TAvSetMmThreadCharacteristicsPtr AvSetMmThreadCharacteristicsPtr = ( TAvSetMmThreadCharacteristicsPtr ) GetProcAddress( AvrtDll, "AvSetMmThreadCharacteristicsW" );
    AvSetMmThreadCharacteristicsPtr( L"Pro Audio", &taskIndex );
    FreeLibrary( AvrtDll );
  }
#endif // PLATFORM_WINDOWS

  // start capture stream if applicable
  if ( captureAudioClient ) {
    hr = captureAudioClient->GetMixFormat( &captureFormat );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve device mix format.";
      goto Exit;
    }

#if PLATFORM_WINDOWS
    // initialize capture stream according to desire buffer size
    REFERENCE_TIME desiredBufferPeriod = ( REFERENCE_TIME ) ( ( float ) stream_.bufferSize * 10000000 / captureFormat->nSamplesPerSec );
#else
	REFERENCE_TIME desiredBufferPeriod = 0;
#endif // PLATFORM_WINDOWS

    if ( !captureClient ) {
      hr = captureAudioClient->Initialize( AUDCLNT_SHAREMODE_SHARED,
                                           AUDCLNT_STREAMFLAGS_EVENTCALLBACK,
                                           desiredBufferPeriod,
                                           desiredBufferPeriod,
                                           captureFormat,
                                           NULL );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to initialize capture audio client.";
        goto Exit;
      }

      hr = captureAudioClient->GetService( __uuidof( IAudioCaptureClient ),
                                           ( void** ) &captureClient );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve capture client handle.";
        goto Exit;
      }

      // configure captureEvent to trigger on every available capture buffer
      captureEvent = CreateEvent( NULL, FALSE, FALSE, NULL );
      if ( !captureEvent ) {
        errorType = RtAudioError::SYSTEM_ERROR;
        errorText_ = "RtApiWasapi::wasapiThread: Unable to create capture event.";
        goto Exit;
      }

      hr = captureAudioClient->SetEventHandle( captureEvent );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to set capture event handle.";
        goto Exit;
      }

      ( ( WasapiHandle* ) stream_.apiHandle )->captureClient = captureClient;
      ( ( WasapiHandle* ) stream_.apiHandle )->captureEvent = captureEvent;
    }

    unsigned int inBufferSize = 0;
    hr = captureAudioClient->GetBufferSize( &inBufferSize );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to get capture buffer size.";
      goto Exit;
    }

    // scale outBufferSize according to stream->user sample rate ratio
    unsigned int outBufferSize = ( unsigned int ) stream_.bufferSize * stream_.nDeviceChannels[INPUT];
    inBufferSize *= stream_.nDeviceChannels[INPUT];

    // set captureBuffer size
    captureBuffer.setBufferSize( inBufferSize + outBufferSize, formatBytes( stream_.deviceFormat[INPUT] ) );

    // reset the capture stream
    hr = captureAudioClient->Reset();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to reset capture stream.";
      goto Exit;
    }

    // start the capture stream
    hr = captureAudioClient->Start();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to start capture stream.";
      goto Exit;
    }
  }

  // start render stream if applicable
  if ( renderAudioClient ) {
    hr = renderAudioClient->GetMixFormat( &renderFormat );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve device mix format.";
      goto Exit;
    }

    // initialize render stream according to desire buffer size
    REFERENCE_TIME desiredBufferPeriod = ( REFERENCE_TIME ) ( ( float ) stream_.bufferSize * 10000000 / renderFormat->nSamplesPerSec );

    if ( !renderClient ) {
      hr = renderAudioClient->Initialize( AUDCLNT_SHAREMODE_SHARED,
                                          AUDCLNT_STREAMFLAGS_EVENTCALLBACK,
                                          desiredBufferPeriod,
                                          desiredBufferPeriod,
                                          renderFormat,
                                          NULL );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to initialize render audio client.";
        goto Exit;
      }

      hr = renderAudioClient->GetService( __uuidof( IAudioRenderClient ),
                                          ( void** ) &renderClient );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve render client handle.";
        goto Exit;
      }

      // configure renderEvent to trigger on every available render buffer
      renderEvent = CreateEvent( NULL, FALSE, FALSE, NULL );
      if ( !renderEvent ) {
        errorType = RtAudioError::SYSTEM_ERROR;
        errorText_ = "RtApiWasapi::wasapiThread: Unable to create render event.";
        goto Exit;
      }

      hr = renderAudioClient->SetEventHandle( renderEvent );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to set render event handle.";
        goto Exit;
      }

      ( ( WasapiHandle* ) stream_.apiHandle )->renderClient = renderClient;
      ( ( WasapiHandle* ) stream_.apiHandle )->renderEvent = renderEvent;
    }

    unsigned int outBufferSize = 0;
    hr = renderAudioClient->GetBufferSize( &outBufferSize );
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to get render buffer size.";
      goto Exit;
    }

    // scale inBufferSize according to user->stream sample rate ratio
    unsigned int inBufferSize = ( unsigned int ) stream_.bufferSize * stream_.nDeviceChannels[OUTPUT];
    outBufferSize *= stream_.nDeviceChannels[OUTPUT];

    // set renderBuffer size
    renderBuffer.setBufferSize( inBufferSize + outBufferSize, formatBytes( stream_.deviceFormat[OUTPUT] ) );

    // reset the render stream
    hr = renderAudioClient->Reset();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to reset render stream.";
      goto Exit;
    }

    // start the render stream
    hr = renderAudioClient->Start();
    if ( FAILED( hr ) ) {
      errorText_ = "RtApiWasapi::wasapiThread: Unable to start render stream.";
      goto Exit;
    }
  }

  if ( stream_.mode == INPUT ) {
    using namespace std; // for roundf
    deviceBuffSize = stream_.bufferSize * stream_.nDeviceChannels[INPUT] * formatBytes( stream_.deviceFormat[INPUT] );
  }
  else if ( stream_.mode == OUTPUT ) {
    deviceBuffSize = stream_.bufferSize * stream_.nDeviceChannels[OUTPUT] * formatBytes( stream_.deviceFormat[OUTPUT] );
  }
  else if ( stream_.mode == DUPLEX ) {
    deviceBuffSize = std::max( stream_.bufferSize * stream_.nDeviceChannels[INPUT] * formatBytes( stream_.deviceFormat[INPUT] ),
                               stream_.bufferSize * stream_.nDeviceChannels[OUTPUT] * formatBytes( stream_.deviceFormat[OUTPUT] ) );
  }

  stream_.deviceBuffer = ( char* ) malloc( deviceBuffSize );
  if ( !stream_.deviceBuffer ) {
    errorType = RtAudioError::MEMORY_ERROR;
    errorText_ = "RtApiWasapi::wasapiThread: Error allocating device buffer memory.";
    goto Exit;
  }

  // stream process loop
  while ( stream_.state != STREAM_STOPPING ) {
    if ( !callbackPulled ) {
      // Callback Input
      // ==============
      // 1. Pull callback buffer from inputBuffer
      // 2. If 1. was successful: Convert callback buffer to user format

      if ( captureAudioClient ) {
        // Pull callback buffer from inputBuffer
        callbackPulled = captureBuffer.pullBuffer( stream_.deviceBuffer,
                                                   ( unsigned int ) stream_.bufferSize * stream_.nDeviceChannels[INPUT],
                                                   stream_.deviceFormat[INPUT] );

        if ( callbackPulled ) {
          if ( stream_.doConvertBuffer[INPUT] ) {
            // Convert callback buffer to user format
            convertBuffer( stream_.userBuffer[INPUT],
                           stream_.deviceBuffer,
                           stream_.convertInfo[INPUT] );
          }
          else {
            // no further conversion, simple copy deviceBuffer to userBuffer
            memcpy( stream_.userBuffer[INPUT],
                    stream_.deviceBuffer,
                    stream_.bufferSize * stream_.nUserChannels[INPUT] * formatBytes( stream_.userFormat ) );
          }
        }
      }
      else {
        // if there is no capture stream, set callbackPulled flag
        callbackPulled = true;
      }

      // Execute Callback
      // ================
      // 1. Execute user callback method
      // 2. Handle return value from callback

      // if callback has not requested the stream to stop
      if ( callbackPulled && !callbackStopped ) {
        // Execute user callback method
        callbackResult = callback( stream_.userBuffer[OUTPUT],
                                   stream_.userBuffer[INPUT],
                                   stream_.bufferSize,
                                   getStreamTime(),
                                   captureFlags & AUDCLNT_BUFFERFLAGS_DATA_DISCONTINUITY ? RTAUDIO_INPUT_OVERFLOW : 0,
                                   stream_.callbackInfo.userData );

        // Handle return value from callback
        if ( callbackResult == 1 ) {
          // instantiate a thread to stop this thread
          HANDLE threadHandle = CreateThread( NULL, 0, stopWasapiThread, this, 0, NULL );
          if ( !threadHandle ) {
            errorType = RtAudioError::THREAD_ERROR;
            errorText_ = "RtApiWasapi::wasapiThread: Unable to instantiate stream stop thread.";
            goto Exit;
          }
          else if ( !CloseHandle( threadHandle ) ) {
            errorType = RtAudioError::THREAD_ERROR;
            errorText_ = "RtApiWasapi::wasapiThread: Unable to close stream stop thread handle.";
            goto Exit;
          }

          callbackStopped = true;
        }
        else if ( callbackResult == 2 ) {
          // instantiate a thread to stop this thread
          HANDLE threadHandle = CreateThread( NULL, 0, abortWasapiThread, this, 0, NULL );
          if ( !threadHandle ) {
            errorType = RtAudioError::THREAD_ERROR;
            errorText_ = "RtApiWasapi::wasapiThread: Unable to instantiate stream abort thread.";
            goto Exit;
          }
          else if ( !CloseHandle( threadHandle ) ) {
            errorType = RtAudioError::THREAD_ERROR;
            errorText_ = "RtApiWasapi::wasapiThread: Unable to close stream abort thread handle.";
            goto Exit;
          }

          callbackStopped = true;
        }
      }
    }

    // Callback Output
    // ===============
    // 1. Convert callback buffer to stream format
    // 2. Push callback buffer into outputBuffer

    if ( renderAudioClient && callbackPulled ) {
      if ( stream_.doConvertBuffer[OUTPUT] ) {
        // Convert callback buffer to stream format
        convertBuffer( stream_.deviceBuffer,
                       stream_.userBuffer[OUTPUT],
                       stream_.convertInfo[OUTPUT] );

      }

      // Push callback buffer into outputBuffer
      callbackPushed = renderBuffer.pushBuffer( stream_.deviceBuffer,
                                                stream_.bufferSize * stream_.nDeviceChannels[OUTPUT],
                                                stream_.deviceFormat[OUTPUT] );
    }
    else {
      // if there is no render stream, set callbackPushed flag
      callbackPushed = true;
    }

    // Stream Capture
    // ==============
    // 1. Get capture buffer from stream
    // 2. Push capture buffer into inputBuffer
    // 3. If 2. was successful: Release capture buffer

    if ( captureAudioClient ) {
      // if the callback input buffer was not pulled from captureBuffer, wait for next capture event
      if ( !callbackPulled ) {
        WaitForSingleObject( captureEvent, INFINITE );
      }

      // Get capture buffer from stream
      hr = captureClient->GetBuffer( &streamBuffer,
                                     &bufferFrameCount,
                                     &captureFlags, NULL, NULL );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve capture buffer.";
        goto Exit;
      }

      if ( bufferFrameCount != 0 ) {
        // Push capture buffer into inputBuffer
        if ( captureBuffer.pushBuffer( ( char* ) streamBuffer,
                                       bufferFrameCount * stream_.nDeviceChannels[INPUT],
                                       stream_.deviceFormat[INPUT] ) )
        {
          // Release capture buffer
          hr = captureClient->ReleaseBuffer( bufferFrameCount );
          if ( FAILED( hr ) ) {
            errorText_ = "RtApiWasapi::wasapiThread: Unable to release capture buffer.";
            goto Exit;
          }
        }
        else
        {
          // Inform WASAPI that capture was unsuccessful
          hr = captureClient->ReleaseBuffer( 0 );
          if ( FAILED( hr ) ) {
            errorText_ = "RtApiWasapi::wasapiThread: Unable to release capture buffer.";
            goto Exit;
          }
        }
      }
      else
      {
        // Inform WASAPI that capture was unsuccessful
        hr = captureClient->ReleaseBuffer( 0 );
        if ( FAILED( hr ) ) {
          errorText_ = "RtApiWasapi::wasapiThread: Unable to release capture buffer.";
          goto Exit;
        }
      }
    }

    // Stream Render
    // =============
    // 1. Get render buffer from stream
    // 2. Pull next buffer from outputBuffer
    // 3. If 2. was successful: Fill render buffer with next buffer
    //                          Release render buffer

    if ( renderAudioClient ) {
      // if the callback output buffer was not pushed to renderBuffer, wait for next render event
      if ( callbackPulled && !callbackPushed ) {
        WaitForSingleObject( renderEvent, INFINITE );
      }

      // Get render buffer from stream
      hr = renderAudioClient->GetBufferSize( &bufferFrameCount );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve render buffer size.";
        goto Exit;
      }

      hr = renderAudioClient->GetCurrentPadding( &numFramesPadding );
      if ( FAILED( hr ) ) {
        errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve render buffer padding.";
        goto Exit;
      }

      bufferFrameCount -= numFramesPadding;

      if ( bufferFrameCount != 0 ) {
        hr = renderClient->GetBuffer( bufferFrameCount, &streamBuffer );
        if ( FAILED( hr ) ) {
          errorText_ = "RtApiWasapi::wasapiThread: Unable to retrieve render buffer.";
          goto Exit;
        }

        // Pull next buffer from outputBuffer
        // Fill render buffer with next buffer
        if ( renderBuffer.pullBuffer( ( char* ) streamBuffer,
                                      bufferFrameCount * stream_.nDeviceChannels[OUTPUT],
                                      stream_.deviceFormat[OUTPUT] ) )
        {
          // Release render buffer
          hr = renderClient->ReleaseBuffer( bufferFrameCount, 0 );
          if ( FAILED( hr ) ) {
            errorText_ = "RtApiWasapi::wasapiThread: Unable to release render buffer.";
            goto Exit;
          }
        }
        else
        {
          // Inform WASAPI that render was unsuccessful
          hr = renderClient->ReleaseBuffer( 0, 0 );
          if ( FAILED( hr ) ) {
            errorText_ = "RtApiWasapi::wasapiThread: Unable to release render buffer.";
            goto Exit;
          }
        }
      }
      else
      {
        // Inform WASAPI that render was unsuccessful
        hr = renderClient->ReleaseBuffer( 0, 0 );
        if ( FAILED( hr ) ) {
          errorText_ = "RtApiWasapi::wasapiThread: Unable to release render buffer.";
          goto Exit;
        }
      }
    }

    // if the callback buffer was pushed renderBuffer reset callbackPulled flag
    if ( callbackPushed ) {
      callbackPulled = false;
      // tick stream time
      RtApi::tickStreamTime();
    }

  }

Exit:
  // clean up
  CoTaskMemFree( captureFormat );
  CoTaskMemFree( renderFormat );

  CoUninitialize();

  // update stream state
  stream_.state = STREAM_STOPPED;

  if ( errorText_.empty() )
    return;
  else
    error( errorType );
}

//******************** End of __WINDOWS_WASAPI__ *********************//
#endif


#if defined(__WINDOWS_DS__) // Windows DirectSound API

// Modified by Robin Davies, October 2005
// - Improvements to DirectX pointer chasing. 
// - Bug fix for non-power-of-two Asio granularity used by Edirol PCR-A30.
// - Auto-call CoInitialize for DSOUND and ASIO platforms.
// Various revisions for RtAudio 4.0 by Gary Scavone, April 2007
// Changed device query structure for RtAudio 4.0.7, January 2010

#include <mmsystem.h>
#include <mmreg.h>
#include <dsound.h>
#include <assert.h>
#include <algorithm>

#if defined(__MINGW32__)
  // missing from latest mingw winapi
#define WAVE_FORMAT_96M08 0x00010000 /* 96 kHz, Mono, 8-bit */
#define WAVE_FORMAT_96S08 0x00020000 /* 96 kHz, Stereo, 8-bit */
#define WAVE_FORMAT_96M16 0x00040000 /* 96 kHz, Mono, 16-bit */
#define WAVE_FORMAT_96S16 0x00080000 /* 96 kHz, Stereo, 16-bit */
#endif

#define MINIMUM_DEVICE_BUFFER_SIZE 32768

#ifdef _MSC_VER // if Microsoft Visual C++
#pragma comment( lib, "winmm.lib" ) // then, auto-link winmm.lib. Otherwise, it has to be added manually.
#endif

static inline DWORD dsPointerBetween( DWORD pointer, DWORD laterPointer, DWORD earlierPointer, DWORD bufferSize )
{
  if ( pointer > bufferSize ) pointer -= bufferSize;
  if ( laterPointer < earlierPointer ) laterPointer += bufferSize;
  if ( pointer < earlierPointer ) pointer += bufferSize;
  return pointer >= earlierPointer && pointer < laterPointer;
}

// A structure to hold various information related to the DirectSound
// API implementation.
struct DsHandle {
  unsigned int drainCounter; // Tracks callback counts when draining
  bool internalDrain;        // Indicates if stop is initiated from callback or not.
  void *id[2];
  void *buffer[2];
  bool xrun[2];
  UINT bufferPointer[2];  
  DWORD dsBufferSize[2];
  DWORD dsPointerLeadTime[2]; // the number of bytes ahead of the safe pointer to lead by.
  HANDLE condition;

  DsHandle()
    :drainCounter(0), internalDrain(false) { id[0] = 0; id[1] = 0; buffer[0] = 0; buffer[1] = 0; xrun[0] = false; xrun[1] = false; bufferPointer[0] = 0; bufferPointer[1] = 0; }
};

// Declarations for utility functions, callbacks, and structures
// specific to the DirectSound implementation.
static BOOL CALLBACK deviceQueryCallback( LPGUID lpguid,
                                          LPCTSTR description,
                                          LPCTSTR module,
                                          LPVOID lpContext );

static const char* getErrorString( int code );

static unsigned __stdcall callbackHandler( void *ptr );

struct DsDevice {
  LPGUID id[2];
  bool validId[2];
  bool found;
  std::string name;

  DsDevice()
  : found(false) { validId[0] = false; validId[1] = false; }
};

struct DsProbeData {
  bool isInput;
  std::vector<struct DsDevice>* dsDevices;
};

RtApiDs :: RtApiDs()
{
  // Dsound will run both-threaded. If CoInitialize fails, then just
  // accept whatever the mainline chose for a threading model.
  coInitialized_ = false;
  HRESULT hr = CoInitialize( NULL );
  if ( !FAILED( hr ) ) coInitialized_ = true;
}

RtApiDs :: ~RtApiDs()
{
  if ( stream_.state != STREAM_CLOSED ) closeStream();
  if ( coInitialized_ ) CoUninitialize(); // balanced call.
}

// The DirectSound default output is always the first device.
unsigned int RtApiDs :: getDefaultOutputDevice( void )
{
  return 0;
}

// The DirectSound default input is always the first input device,
// which is the first capture device enumerated.
unsigned int RtApiDs :: getDefaultInputDevice( void )
{
  return 0;
}

unsigned int RtApiDs :: getDeviceCount( void )
{
  // Set query flag for previously found devices to false, so that we
  // can check for any devices that have disappeared.
  for ( unsigned int i=0; i<dsDevices.size(); i++ )
    dsDevices[i].found = false;

  // Query DirectSound devices.
  struct DsProbeData probeInfo;
  probeInfo.isInput = false;
  probeInfo.dsDevices = &dsDevices;
  HRESULT result = DirectSoundEnumerate( (LPDSENUMCALLBACK) deviceQueryCallback, &probeInfo );
  if ( FAILED( result ) ) {
    errorStream_ << "RtApiDs::getDeviceCount: error (" << getErrorString( result ) << ") enumerating output devices!";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
  }

  // Query DirectSoundCapture devices.
  probeInfo.isInput = true;
  result = DirectSoundCaptureEnumerate( (LPDSENUMCALLBACK) deviceQueryCallback, &probeInfo );
  if ( FAILED( result ) ) {
    errorStream_ << "RtApiDs::getDeviceCount: error (" << getErrorString( result ) << ") enumerating input devices!";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
  }

  // Clean out any devices that may have disappeared (code update submitted by Eli Zehngut).
  for ( unsigned int i=0; i<dsDevices.size(); ) {
    if ( dsDevices[i].found == false ) dsDevices.erase( dsDevices.begin() + i );
    else i++;
  }

  return static_cast<unsigned int>(dsDevices.size());
}

RtAudio::DeviceInfo RtApiDs :: getDeviceInfo( unsigned int device )
{
  RtAudio::DeviceInfo info;
  info.probed = false;

  if ( dsDevices.size() == 0 ) {
    // Force a query of all devices
    getDeviceCount();
    if ( dsDevices.size() == 0 ) {
      errorText_ = "RtApiDs::getDeviceInfo: no devices found!";
      error( RtAudioError::INVALID_USE );
      return info;
    }
  }

  if ( device >= dsDevices.size() ) {
    errorText_ = "RtApiDs::getDeviceInfo: device ID is invalid!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  HRESULT result;
  if ( dsDevices[ device ].validId[0] == false ) goto probeInput;

  LPDIRECTSOUND output;
  DSCAPS outCaps;
  result = DirectSoundCreate( dsDevices[ device ].id[0], &output, NULL );
  if ( FAILED( result ) ) {
    errorStream_ << "RtApiDs::getDeviceInfo: error (" << getErrorString( result ) << ") opening output device (" << dsDevices[ device ].name << ")!";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    goto probeInput;
  }

  outCaps.dwSize = sizeof( outCaps );
  result = output->GetCaps( &outCaps );
  if ( FAILED( result ) ) {
    output->Release();
    errorStream_ << "RtApiDs::getDeviceInfo: error (" << getErrorString( result ) << ") getting capabilities!";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    goto probeInput;
  }

  // Get output channel information.
  info.outputChannels = ( outCaps.dwFlags & DSCAPS_PRIMARYSTEREO ) ? 2 : 1;

  // Get sample rate information.
  info.sampleRates.clear();
  for ( unsigned int k=0; k<MAX_SAMPLE_RATES; k++ ) {
    if ( SAMPLE_RATES[k] >= (unsigned int) outCaps.dwMinSecondarySampleRate &&
         SAMPLE_RATES[k] <= (unsigned int) outCaps.dwMaxSecondarySampleRate ) {
      info.sampleRates.push_back( SAMPLE_RATES[k] );

      if ( !info.preferredSampleRate || ( SAMPLE_RATES[k] <= 48000 && SAMPLE_RATES[k] > info.preferredSampleRate ) )
        info.preferredSampleRate = SAMPLE_RATES[k];
    }
  }

  // Get format information.
  if ( outCaps.dwFlags & DSCAPS_PRIMARY16BIT ) info.nativeFormats |= RTAUDIO_SINT16;
  if ( outCaps.dwFlags & DSCAPS_PRIMARY8BIT ) info.nativeFormats |= RTAUDIO_SINT8;

  output->Release();

  if ( getDefaultOutputDevice() == device )
    info.isDefaultOutput = true;

  if ( dsDevices[ device ].validId[1] == false ) {
    info.name = dsDevices[ device ].name;
	info.deviceId = convertGuidToStdString(dsDevices[device].id[0]);
    info.probed = true;
    return info;
  }

 probeInput:

  LPDIRECTSOUNDCAPTURE input;
  result = DirectSoundCaptureCreate( dsDevices[ device ].id[1], &input, NULL );
  if ( FAILED( result ) ) {
    errorStream_ << "RtApiDs::getDeviceInfo: error (" << getErrorString( result ) << ") opening input device (" << dsDevices[ device ].name << ")!";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  DSCCAPS inCaps;
  inCaps.dwSize = sizeof( inCaps );
  result = input->GetCaps( &inCaps );
  if ( FAILED( result ) ) {
    input->Release();
    errorStream_ << "RtApiDs::getDeviceInfo: error (" << getErrorString( result ) << ") getting object capabilities (" << dsDevices[ device ].name << ")!";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Get input channel information.
  info.inputChannels = inCaps.dwChannels;

  // Get sample rate and format information.
  std::vector<unsigned int> rates;
  if ( inCaps.dwChannels >= 2 ) {
    if ( inCaps.dwFormats & WAVE_FORMAT_1S16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_2S16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_4S16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_96S16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_1S08 ) info.nativeFormats |= RTAUDIO_SINT8;
    if ( inCaps.dwFormats & WAVE_FORMAT_2S08 ) info.nativeFormats |= RTAUDIO_SINT8;
    if ( inCaps.dwFormats & WAVE_FORMAT_4S08 ) info.nativeFormats |= RTAUDIO_SINT8;
	if ( inCaps.dwFormats & WAVE_FORMAT_48S08 ) info.nativeFormats |= RTAUDIO_SINT8;
    if ( inCaps.dwFormats & WAVE_FORMAT_96S08 ) info.nativeFormats |= RTAUDIO_SINT8;

    if ( info.nativeFormats & RTAUDIO_SINT16 ) {
      if ( inCaps.dwFormats & WAVE_FORMAT_1S16 ) rates.push_back( 11025 );
      if ( inCaps.dwFormats & WAVE_FORMAT_2S16 ) rates.push_back( 22050 );
      if ( inCaps.dwFormats & WAVE_FORMAT_4S16 ) rates.push_back( 44100 );
	  if ( inCaps.dwFormats & WAVE_FORMAT_48S16 ) rates.push_back( 48000 );
      if ( inCaps.dwFormats & WAVE_FORMAT_96S16 ) rates.push_back( 96000 );
    }
    else if ( info.nativeFormats & RTAUDIO_SINT8 ) {
      if ( inCaps.dwFormats & WAVE_FORMAT_1S08 ) rates.push_back( 11025 );
      if ( inCaps.dwFormats & WAVE_FORMAT_2S08 ) rates.push_back( 22050 );
      if ( inCaps.dwFormats & WAVE_FORMAT_4S08 ) rates.push_back( 44100 );
	  if ( inCaps.dwFormats & WAVE_FORMAT_48S08 ) rates.push_back( 48000 );
      if ( inCaps.dwFormats & WAVE_FORMAT_96S08 ) rates.push_back( 96000 );
    }
  }
  else if ( inCaps.dwChannels == 1 ) {
    if ( inCaps.dwFormats & WAVE_FORMAT_1M16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_2M16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_4M16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_96M16 ) info.nativeFormats |= RTAUDIO_SINT16;
    if ( inCaps.dwFormats & WAVE_FORMAT_1M08 ) info.nativeFormats |= RTAUDIO_SINT8;
    if ( inCaps.dwFormats & WAVE_FORMAT_2M08 ) info.nativeFormats |= RTAUDIO_SINT8;
    if ( inCaps.dwFormats & WAVE_FORMAT_4M08 ) info.nativeFormats |= RTAUDIO_SINT8;
	if ( inCaps.dwFormats & WAVE_FORMAT_48M08 ) info.nativeFormats |= RTAUDIO_SINT8;
    if ( inCaps.dwFormats & WAVE_FORMAT_96M08 ) info.nativeFormats |= RTAUDIO_SINT8;

    if ( info.nativeFormats & RTAUDIO_SINT16 ) {
      if ( inCaps.dwFormats & WAVE_FORMAT_1M16 ) rates.push_back( 11025 );
      if ( inCaps.dwFormats & WAVE_FORMAT_2M16 ) rates.push_back( 22050 );
      if ( inCaps.dwFormats & WAVE_FORMAT_4M16 ) rates.push_back( 44100 );
	  if ( inCaps.dwFormats & WAVE_FORMAT_48M16 ) rates.push_back( 48000 );
      if ( inCaps.dwFormats & WAVE_FORMAT_96M16 ) rates.push_back( 96000 );
    }
    else if ( info.nativeFormats & RTAUDIO_SINT8 ) {
      if ( inCaps.dwFormats & WAVE_FORMAT_1M08 ) rates.push_back( 11025 );
      if ( inCaps.dwFormats & WAVE_FORMAT_2M08 ) rates.push_back( 22050 );
      if ( inCaps.dwFormats & WAVE_FORMAT_4M08 ) rates.push_back( 44100 );
	  if ( inCaps.dwFormats & WAVE_FORMAT_48M08) rates.push_back( 48000 );
      if ( inCaps.dwFormats & WAVE_FORMAT_96M08 ) rates.push_back( 96000 );
    }
  }
  else info.inputChannels = 0; // technically, this would be an error

  input->Release();

  if ( info.inputChannels == 0 ) return info;

  // Copy the supported rates to the info structure but avoid duplication.
  bool found;
  for ( unsigned int i=0; i<rates.size(); i++ ) {
    found = false;
    for ( unsigned int j=0; j<info.sampleRates.size(); j++ ) {
      if ( rates[i] == info.sampleRates[j] ) {
        found = true;
        break;
      }
    }
    if ( found == false ) info.sampleRates.push_back( rates[i] );
  }
  std::sort( info.sampleRates.begin(), info.sampleRates.end() );
  
  // Get preferred sample rate
  for (unsigned int i = 0; i < info.sampleRates.size(); i++) {
	  if (info.sampleRates[i] > info.preferredSampleRate && info.preferredSampleRate < 48000) {
		  info.preferredSampleRate = info.sampleRates[i];
	  }
  }

  // If device opens for both playback and capture, we determine the channels.
  if ( info.outputChannels > 0 && info.inputChannels > 0 )
    info.duplexChannels = (info.outputChannels > info.inputChannels) ? info.inputChannels : info.outputChannels;

  if ( device == 0 ) info.isDefaultInput = true;

  // Copy name and return.
  info.name = dsDevices[ device ].name;
  info.deviceId = convertGuidToStdString(dsDevices[device].id[1]);
  info.probed = true;
  return info;
}

bool RtApiDs :: probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                                 unsigned int firstChannel, unsigned int sampleRate,
                                 RtAudioFormat format, unsigned int *bufferSize,
                                 RtAudio::StreamOptions *options )
{
  if ( channels + firstChannel > 2 ) {
    errorText_ = "RtApiDs::probeDeviceOpen: DirectSound does not support more than 2 channels per device.";
    return FAILURE;
  }

  size_t nDevices = dsDevices.size();
  if ( nDevices == 0 ) {
    // This should not happen because a check is made before this function is called.
    errorText_ = "RtApiDs::probeDeviceOpen: no devices found!";
    return FAILURE;
  }

  if ( device >= nDevices ) {
    // This should not happen because a check is made before this function is called.
    errorText_ = "RtApiDs::probeDeviceOpen: device ID is invalid!";
    return FAILURE;
  }

  if ( mode == OUTPUT ) {
    if ( dsDevices[ device ].validId[0] == false ) {
      errorStream_ << "RtApiDs::probeDeviceOpen: device (" << device << ") does not support output!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }
  }
  else { // mode == INPUT
    if ( dsDevices[ device ].validId[1] == false ) {
      errorStream_ << "RtApiDs::probeDeviceOpen: device (" << device << ") does not support input!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }
  }

  // According to a note in PortAudio, using GetDesktopWindow()
  // instead of GetForegroundWindow() is supposed to avoid problems
  // that occur when the application's window is not the foreground
  // window.  Also, if the application window closes before the
  // DirectSound buffer, DirectSound can crash.  In the past, I had
  // problems when using GetDesktopWindow() but it seems fine now
  // (January 2010).  I'll leave it commented here.
  // HWND hWnd = GetForegroundWindow();
  HWND hWnd = GetDesktopWindow();

  // Check the numberOfBuffers parameter and limit the lowest value to
  // two.  This is a judgement call and a value of two is probably too
  // low for capture, but it should work for playback.
  int nBuffers = 0;
  if ( options ) nBuffers = options->numberOfBuffers;
  if ( options && options->flags & RTAUDIO_MINIMIZE_LATENCY ) nBuffers = 2;
  if ( nBuffers < 2 ) nBuffers = 3;

  // Check the lower range of the user-specified buffer size and set
  // (arbitrarily) to a lower bound of 32.
  if ( *bufferSize < 32 ) *bufferSize = 32;

  // Create the wave format structure.  The data format setting will
  // be determined later.
  WAVEFORMATEX waveFormat;
  ZeroMemory( &waveFormat, sizeof(WAVEFORMATEX) );
  waveFormat.wFormatTag = WAVE_FORMAT_PCM;
  waveFormat.nChannels = channels + firstChannel;
  waveFormat.nSamplesPerSec = (unsigned long) sampleRate;

  // Determine the device buffer size. By default, we'll use the value
  // defined above (32K), but we will grow it to make allowances for
  // very large software buffer sizes.
  DWORD dsBufferSize = MINIMUM_DEVICE_BUFFER_SIZE;
  DWORD dsPointerLeadTime = 0;

  void *ohandle = 0, *bhandle = 0;
  HRESULT result;
  if ( mode == OUTPUT ) {

    LPDIRECTSOUND output;
    result = DirectSoundCreate( dsDevices[ device ].id[0], &output, NULL );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") opening output device (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    DSCAPS outCaps;
    outCaps.dwSize = sizeof( outCaps );
    result = output->GetCaps( &outCaps );
    if ( FAILED( result ) ) {
      output->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") getting capabilities (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Check channel information.
    if ( channels + firstChannel == 2 && !( outCaps.dwFlags & DSCAPS_PRIMARYSTEREO ) ) {
      errorStream_ << "RtApiDs::getDeviceInfo: the output device (" << dsDevices[ device ].name << ") does not support stereo playback.";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Check format information.  Use 16-bit format unless not
    // supported or user requests 8-bit.
    if ( outCaps.dwFlags & DSCAPS_PRIMARY16BIT &&
         !( format == RTAUDIO_SINT8 && outCaps.dwFlags & DSCAPS_PRIMARY8BIT ) ) {
      waveFormat.wBitsPerSample = 16;
      stream_.deviceFormat[mode] = RTAUDIO_SINT16;
    }
    else {
      waveFormat.wBitsPerSample = 8;
      stream_.deviceFormat[mode] = RTAUDIO_SINT8;
    }
    stream_.userFormat = format;

    // Update wave format structure and buffer information.
    waveFormat.nBlockAlign = waveFormat.nChannels * waveFormat.wBitsPerSample / 8;
    waveFormat.nAvgBytesPerSec = waveFormat.nSamplesPerSec * waveFormat.nBlockAlign;
    dsPointerLeadTime = nBuffers * (*bufferSize) * (waveFormat.wBitsPerSample / 8) * channels;

    // If the user wants an even bigger buffer, increase the device buffer size accordingly.
    while ( dsPointerLeadTime * 2U > dsBufferSize )
      dsBufferSize *= 2;

    // Set cooperative level to DSSCL_EXCLUSIVE ... sound stops when window focus changes.
    // result = output->SetCooperativeLevel( hWnd, DSSCL_EXCLUSIVE );
    // Set cooperative level to DSSCL_PRIORITY ... sound remains when window focus changes.
    result = output->SetCooperativeLevel( hWnd, DSSCL_PRIORITY );
    if ( FAILED( result ) ) {
      output->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") setting cooperative level (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Even though we will write to the secondary buffer, we need to
    // access the primary buffer to set the correct output format
    // (since the default is 8-bit, 22 kHz!).  Setup the DS primary
    // buffer description.
    DSBUFFERDESC bufferDescription;
    ZeroMemory( &bufferDescription, sizeof( DSBUFFERDESC ) );
    bufferDescription.dwSize = sizeof( DSBUFFERDESC );
    bufferDescription.dwFlags = DSBCAPS_PRIMARYBUFFER;

    // Obtain the primary buffer
    LPDIRECTSOUNDBUFFER buffer;
    result = output->CreateSoundBuffer( &bufferDescription, &buffer, NULL );
    if ( FAILED( result ) ) {
      output->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") accessing primary buffer (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Set the primary DS buffer sound format.
    result = buffer->SetFormat( &waveFormat );
    if ( FAILED( result ) ) {
      output->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") setting primary buffer format (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Setup the secondary DS buffer description.
    ZeroMemory( &bufferDescription, sizeof( DSBUFFERDESC ) );
    bufferDescription.dwSize = sizeof( DSBUFFERDESC );
    bufferDescription.dwFlags = ( DSBCAPS_STICKYFOCUS |
                                  DSBCAPS_GLOBALFOCUS |
                                  DSBCAPS_GETCURRENTPOSITION2 |
                                  DSBCAPS_LOCHARDWARE );  // Force hardware mixing
    bufferDescription.dwBufferBytes = dsBufferSize;
    bufferDescription.lpwfxFormat = &waveFormat;

    // Try to create the secondary DS buffer.  If that doesn't work,
    // try to use software mixing.  Otherwise, there's a problem.
    result = output->CreateSoundBuffer( &bufferDescription, &buffer, NULL );
    if ( FAILED( result ) ) {
      bufferDescription.dwFlags = ( DSBCAPS_STICKYFOCUS |
                                    DSBCAPS_GLOBALFOCUS |
                                    DSBCAPS_GETCURRENTPOSITION2 |
                                    DSBCAPS_LOCSOFTWARE );  // Force software mixing
      result = output->CreateSoundBuffer( &bufferDescription, &buffer, NULL );
      if ( FAILED( result ) ) {
        output->Release();
        errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") creating secondary buffer (" << dsDevices[ device ].name << ")!";
        errorText_ = errorStream_.str();
        return FAILURE;
      }
    }

    // Get the buffer size ... might be different from what we specified.
    DSBCAPS dsbcaps;
    dsbcaps.dwSize = sizeof( DSBCAPS );
    result = buffer->GetCaps( &dsbcaps );
    if ( FAILED( result ) ) {
      output->Release();
      buffer->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") getting buffer settings (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    dsBufferSize = dsbcaps.dwBufferBytes;

    // Lock the DS buffer
    LPVOID audioPtr;
    DWORD dataLen;
    result = buffer->Lock( 0, dsBufferSize, &audioPtr, &dataLen, NULL, NULL, 0 );
    if ( FAILED( result ) ) {
      output->Release();
      buffer->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") locking buffer (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Zero the DS buffer
    ZeroMemory( audioPtr, dataLen );

    // Unlock the DS buffer
    result = buffer->Unlock( audioPtr, dataLen, NULL, 0 );
    if ( FAILED( result ) ) {
      output->Release();
      buffer->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") unlocking buffer (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    ohandle = (void *) output;
    bhandle = (void *) buffer;
  }

  if ( mode == INPUT ) {

    LPDIRECTSOUNDCAPTURE input;
    result = DirectSoundCaptureCreate( dsDevices[ device ].id[1], &input, NULL );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") opening input device (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    DSCCAPS inCaps;
    inCaps.dwSize = sizeof( inCaps );
    result = input->GetCaps( &inCaps );
    if ( FAILED( result ) ) {
      input->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") getting input capabilities (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Check channel information.
    if ( inCaps.dwChannels < channels + firstChannel ) {
      errorText_ = "RtApiDs::getDeviceInfo: the input device does not support requested input channels.";
      return FAILURE;
    }

    // Check format information.  Use 16-bit format unless user
    // requests 8-bit.
    DWORD deviceFormats;
    if ( channels + firstChannel == 2 ) {
      deviceFormats = WAVE_FORMAT_1S08 | WAVE_FORMAT_2S08 | WAVE_FORMAT_4S08 | WAVE_FORMAT_96S08;
      if ( format == RTAUDIO_SINT8 && inCaps.dwFormats & deviceFormats ) {
        waveFormat.wBitsPerSample = 8;
        stream_.deviceFormat[mode] = RTAUDIO_SINT8;
      }
      else { // assume 16-bit is supported
        waveFormat.wBitsPerSample = 16;
        stream_.deviceFormat[mode] = RTAUDIO_SINT16;
      }
    }
    else { // channel == 1
      deviceFormats = WAVE_FORMAT_1M08 | WAVE_FORMAT_2M08 | WAVE_FORMAT_4M08 | WAVE_FORMAT_96M08;
      if ( format == RTAUDIO_SINT8 && inCaps.dwFormats & deviceFormats ) {
        waveFormat.wBitsPerSample = 8;
        stream_.deviceFormat[mode] = RTAUDIO_SINT8;
      }
      else { // assume 16-bit is supported
        waveFormat.wBitsPerSample = 16;
        stream_.deviceFormat[mode] = RTAUDIO_SINT16;
      }
    }
    stream_.userFormat = format;

    // Update wave format structure and buffer information.
    waveFormat.nBlockAlign = waveFormat.nChannels * waveFormat.wBitsPerSample / 8;
    waveFormat.nAvgBytesPerSec = waveFormat.nSamplesPerSec * waveFormat.nBlockAlign;
    dsPointerLeadTime = nBuffers * (*bufferSize) * (waveFormat.wBitsPerSample / 8) * channels;

    // If the user wants an even bigger buffer, increase the device buffer size accordingly.
    while ( dsPointerLeadTime * 2U > dsBufferSize )
      dsBufferSize *= 2;

    // Setup the secondary DS buffer description.
    DSCBUFFERDESC bufferDescription;
    ZeroMemory( &bufferDescription, sizeof( DSCBUFFERDESC ) );
    bufferDescription.dwSize = sizeof( DSCBUFFERDESC );
    bufferDescription.dwFlags = 0;
    bufferDescription.dwReserved = 0;
    bufferDescription.dwBufferBytes = dsBufferSize;
    bufferDescription.lpwfxFormat = &waveFormat;

    // Create the capture buffer.
    LPDIRECTSOUNDCAPTUREBUFFER buffer;
    result = input->CreateCaptureBuffer( &bufferDescription, &buffer, NULL );
    if ( FAILED( result ) ) {
      input->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") creating input buffer (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Get the buffer size ... might be different from what we specified.
    DSCBCAPS dscbcaps;
    dscbcaps.dwSize = sizeof( DSCBCAPS );
    result = buffer->GetCaps( &dscbcaps );
    if ( FAILED( result ) ) {
      input->Release();
      buffer->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") getting buffer settings (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    dsBufferSize = dscbcaps.dwBufferBytes;

    // NOTE: We could have a problem here if this is a duplex stream
    // and the play and capture hardware buffer sizes are different
    // (I'm actually not sure if that is a problem or not).
    // Currently, we are not verifying that.

    // Lock the capture buffer
    LPVOID audioPtr;
    DWORD dataLen;
    result = buffer->Lock( 0, dsBufferSize, &audioPtr, &dataLen, NULL, NULL, 0 );
    if ( FAILED( result ) ) {
      input->Release();
      buffer->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") locking input buffer (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    // Zero the buffer
    ZeroMemory( audioPtr, dataLen );

    // Unlock the buffer
    result = buffer->Unlock( audioPtr, dataLen, NULL, 0 );
    if ( FAILED( result ) ) {
      input->Release();
      buffer->Release();
      errorStream_ << "RtApiDs::probeDeviceOpen: error (" << getErrorString( result ) << ") unlocking input buffer (" << dsDevices[ device ].name << ")!";
      errorText_ = errorStream_.str();
      return FAILURE;
    }

    ohandle = (void *) input;
    bhandle = (void *) buffer;
  }

  // Set various stream parameters
  DsHandle *handle = 0;
  stream_.nDeviceChannels[mode] = channels + firstChannel;
  stream_.nUserChannels[mode] = channels;
  stream_.bufferSize = *bufferSize;
  stream_.channelOffset[mode] = firstChannel;
  stream_.deviceInterleaved[mode] = true;
  if ( options && options->flags & RTAUDIO_NONINTERLEAVED ) stream_.userInterleaved = false;
  else stream_.userInterleaved = true;

  // Set flag for buffer conversion
  stream_.doConvertBuffer[mode] = false;
  if (stream_.nUserChannels[mode] != stream_.nDeviceChannels[mode])
    stream_.doConvertBuffer[mode] = true;
  if (stream_.userFormat != stream_.deviceFormat[mode])
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.userInterleaved != stream_.deviceInterleaved[mode] &&
       stream_.nUserChannels[mode] > 1 )
    stream_.doConvertBuffer[mode] = true;

  // Allocate necessary internal buffers
  long bufferBytes = stream_.nUserChannels[mode] * *bufferSize * formatBytes( stream_.userFormat );
  stream_.userBuffer[mode] = (char *) calloc( bufferBytes, 1 );
  if ( stream_.userBuffer[mode] == NULL ) {
    errorText_ = "RtApiDs::probeDeviceOpen: error allocating user buffer memory.";
    goto error;
  }

  if ( stream_.doConvertBuffer[mode] ) {

    bool makeBuffer = true;
    bufferBytes = stream_.nDeviceChannels[mode] * formatBytes( stream_.deviceFormat[mode] );
    if ( mode == INPUT ) {
      if ( stream_.mode == OUTPUT && stream_.deviceBuffer ) {
        unsigned long bytesOut = stream_.nDeviceChannels[0] * formatBytes( stream_.deviceFormat[0] );
        if ( bufferBytes <= (long) bytesOut ) makeBuffer = false;
      }
    }

    if ( makeBuffer ) {
      bufferBytes *= *bufferSize;
      if ( stream_.deviceBuffer ) free( stream_.deviceBuffer );
      stream_.deviceBuffer = (char *) calloc( bufferBytes, 1 );
      if ( stream_.deviceBuffer == NULL ) {
        errorText_ = "RtApiDs::probeDeviceOpen: error allocating device buffer memory.";
        goto error;
      }
    }
  }

  // Allocate our DsHandle structures for the stream.
  if ( stream_.apiHandle == 0 ) {
     handle = new DsHandle;

    // Create a manual-reset event.
    handle->condition = CreateEvent( NULL,   // no security
                                     TRUE,   // manual-reset
                                     FALSE,  // non-signaled initially
                                     NULL ); // unnamed
    stream_.apiHandle = (void *) handle;
  }
  else
    handle = (DsHandle *) stream_.apiHandle;
  handle->id[mode] = ohandle;
  handle->buffer[mode] = bhandle;
  handle->dsBufferSize[mode] = dsBufferSize;
  handle->dsPointerLeadTime[mode] = dsPointerLeadTime;

  stream_.device[mode] = device;
  stream_.state = STREAM_STOPPED;
  if ( stream_.mode == OUTPUT && mode == INPUT )
    // We had already set up an output stream.
    stream_.mode = DUPLEX;
  else
    stream_.mode = mode;
  stream_.nBuffers = nBuffers;
  stream_.sampleRate = sampleRate;

  // Setup the buffer conversion information structure.
  if ( stream_.doConvertBuffer[mode] ) setConvertInfo( mode, firstChannel );

  // Setup the callback thread.
  if ( stream_.callbackInfo.isRunning == false ) {
    unsigned threadId;
    stream_.callbackInfo.isRunning = true;
    stream_.callbackInfo.object = (void *) this;
    stream_.callbackInfo.thread = _beginthreadex( NULL, 0, &callbackHandler,
                                                  &stream_.callbackInfo, 0, &threadId );
    if ( stream_.callbackInfo.thread == 0 ) {
      errorText_ = "RtApiDs::probeDeviceOpen: error creating callback thread!";
      goto error;
    }

    // Boost DS thread priority
    SetThreadPriority( (HANDLE) stream_.callbackInfo.thread, THREAD_PRIORITY_HIGHEST );
  }
  return SUCCESS;

 error:
  if ( handle ) {
    if ( handle->buffer[0] ) { // the object pointer can be NULL and valid
      LPDIRECTSOUND object = (LPDIRECTSOUND) handle->id[0];
      LPDIRECTSOUNDBUFFER buffer = (LPDIRECTSOUNDBUFFER) handle->buffer[0];
      if ( buffer ) buffer->Release();
      object->Release();
    }
    if ( handle->buffer[1] ) {
      LPDIRECTSOUNDCAPTURE object = (LPDIRECTSOUNDCAPTURE) handle->id[1];
      LPDIRECTSOUNDCAPTUREBUFFER buffer = (LPDIRECTSOUNDCAPTUREBUFFER) handle->buffer[1];
      if ( buffer ) buffer->Release();
      object->Release();
    }
    CloseHandle( handle->condition );
    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.state = STREAM_CLOSED;
  return FAILURE;
}

void RtApiDs :: closeStream()
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiDs::closeStream(): no open stream to close!";
    error( RtAudioError::WARNING );
    return;
  }

  // Stop the callback thread.
  stream_.callbackInfo.isRunning = false;
  WaitForSingleObject( (HANDLE) stream_.callbackInfo.thread, INFINITE );
  CloseHandle( (HANDLE) stream_.callbackInfo.thread );

  DsHandle *handle = (DsHandle *) stream_.apiHandle;
  if ( handle ) {
    if ( handle->buffer[0] ) { // the object pointer can be NULL and valid
      LPDIRECTSOUND object = (LPDIRECTSOUND) handle->id[0];
      LPDIRECTSOUNDBUFFER buffer = (LPDIRECTSOUNDBUFFER) handle->buffer[0];
      if ( buffer ) {
        buffer->Stop();
        buffer->Release();
      }
      object->Release();
    }
    if ( handle->buffer[1] ) {
      LPDIRECTSOUNDCAPTURE object = (LPDIRECTSOUNDCAPTURE) handle->id[1];
      LPDIRECTSOUNDCAPTUREBUFFER buffer = (LPDIRECTSOUNDCAPTUREBUFFER) handle->buffer[1];
      if ( buffer ) {
        buffer->Stop();
        buffer->Release();
      }
      object->Release();
    }
    CloseHandle( handle->condition );
    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.mode = UNINITIALIZED;
  stream_.state = STREAM_CLOSED;
}

void RtApiDs :: startStream()
{
  verifyStream();
  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiDs::startStream(): the stream is already running!";
    error( RtAudioError::WARNING );
    return;
  }

  DsHandle *handle = (DsHandle *) stream_.apiHandle;

  // Increase scheduler frequency on lesser windows (a side-effect of
  // increasing timer accuracy).  On greater windows (Win2K or later),
  // this is already in effect.
  timeBeginPeriod( 1 ); 

  buffersRolling = false;
  duplexPrerollBytes = 0;

  if ( stream_.mode == DUPLEX ) {
    // 0.5 seconds of silence in DUPLEX mode while the devices spin up and synchronize.
    duplexPrerollBytes = (int) ( 0.5 * stream_.sampleRate * formatBytes( stream_.deviceFormat[1] ) * stream_.nDeviceChannels[1] );
  }

  HRESULT result = 0;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    LPDIRECTSOUNDBUFFER buffer = (LPDIRECTSOUNDBUFFER) handle->buffer[0];
    result = buffer->Play( 0, 0, DSBPLAY_LOOPING );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::startStream: error (" << getErrorString( result ) << ") starting output buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX ) {

    LPDIRECTSOUNDCAPTUREBUFFER buffer = (LPDIRECTSOUNDCAPTUREBUFFER) handle->buffer[1];
    result = buffer->Start( DSCBSTART_LOOPING );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::startStream: error (" << getErrorString( result ) << ") starting input buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  handle->drainCounter = 0;
  handle->internalDrain = false;
  ResetEvent( handle->condition );
  stream_.state = STREAM_RUNNING;

 unlock:
  if ( FAILED( result ) ) error( RtAudioError::SYSTEM_ERROR );
}

void RtApiDs :: stopStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiDs::stopStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  HRESULT result = 0;
  LPVOID audioPtr;
  DWORD dataLen;
  DsHandle *handle = (DsHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    if ( handle->drainCounter == 0 ) {
      handle->drainCounter = 2;
      WaitForSingleObject( handle->condition, INFINITE );  // block until signaled
    }

    stream_.state = STREAM_STOPPED;

    MUTEX_LOCK( &stream_.mutex );

    // Stop the buffer and clear memory
    LPDIRECTSOUNDBUFFER buffer = (LPDIRECTSOUNDBUFFER) handle->buffer[0];
    result = buffer->Stop();
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::stopStream: error (" << getErrorString( result ) << ") stopping output buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }

    // Lock the buffer and clear it so that if we start to play again,
    // we won't have old data playing.
    result = buffer->Lock( 0, handle->dsBufferSize[0], &audioPtr, &dataLen, NULL, NULL, 0 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::stopStream: error (" << getErrorString( result ) << ") locking output buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }

    // Zero the DS buffer
    ZeroMemory( audioPtr, dataLen );

    // Unlock the DS buffer
    result = buffer->Unlock( audioPtr, dataLen, NULL, 0 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::stopStream: error (" << getErrorString( result ) << ") unlocking output buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }

    // If we start playing again, we must begin at beginning of buffer.
    handle->bufferPointer[0] = 0;
  }

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX ) {
    LPDIRECTSOUNDCAPTUREBUFFER buffer = (LPDIRECTSOUNDCAPTUREBUFFER) handle->buffer[1];
    audioPtr = NULL;
    dataLen = 0;

    stream_.state = STREAM_STOPPED;

    if ( stream_.mode != DUPLEX )
      MUTEX_LOCK( &stream_.mutex );

    result = buffer->Stop();
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::stopStream: error (" << getErrorString( result ) << ") stopping input buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }

    // Lock the buffer and clear it so that if we start to play again,
    // we won't have old data playing.
    result = buffer->Lock( 0, handle->dsBufferSize[1], &audioPtr, &dataLen, NULL, NULL, 0 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::stopStream: error (" << getErrorString( result ) << ") locking input buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }

    // Zero the DS buffer
    ZeroMemory( audioPtr, dataLen );

    // Unlock the DS buffer
    result = buffer->Unlock( audioPtr, dataLen, NULL, 0 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::stopStream: error (" << getErrorString( result ) << ") unlocking input buffer!";
      errorText_ = errorStream_.str();
      goto unlock;
    }

    // If we start recording again, we must begin at beginning of buffer.
    handle->bufferPointer[1] = 0;
  }

 unlock:
  timeEndPeriod( 1 ); // revert to normal scheduler frequency on lesser windows.
  MUTEX_UNLOCK( &stream_.mutex );

  if ( FAILED( result ) ) error( RtAudioError::SYSTEM_ERROR );
}

void RtApiDs :: abortStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiDs::abortStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  DsHandle *handle = (DsHandle *) stream_.apiHandle;
  handle->drainCounter = 2;

  stopStream();
}

#pragma warning ( push )
#pragma warning ( disable : 6385 ) // MSVC has issue with memcpy https://developercommunity.visualstudio.com/content/problem/841208/false-c6385-warning-when-using-memcpy.html

void RtApiDs :: callbackEvent()
{
  if ( stream_.state == STREAM_STOPPED || stream_.state == STREAM_STOPPING ) {
    Sleep( 50 ); // sleep 50 milliseconds
    return;
  }

  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiDs::callbackEvent(): the stream is closed ... this shouldn't happen!";
    error( RtAudioError::WARNING );
    return;
  }

  CallbackInfo *info = (CallbackInfo *) &stream_.callbackInfo;
  DsHandle *handle = (DsHandle *) stream_.apiHandle;

  // Check if we were draining the stream and signal is finished.
  if ( handle->drainCounter > stream_.nBuffers + 2 ) {

    stream_.state = STREAM_STOPPING;
    if ( handle->internalDrain == false )
      SetEvent( handle->condition );
    else
      stopStream();
    return;
  }

  // Invoke user callback to get fresh output data UNLESS we are
  // draining stream.
  if ( handle->drainCounter == 0 ) {
    RtAudioCallback callback = (RtAudioCallback) info->callback;
    double streamTime = getStreamTime();
    RtAudioStreamStatus status = 0;
    if ( stream_.mode != INPUT && handle->xrun[0] == true ) {
      status |= RTAUDIO_OUTPUT_UNDERFLOW;
      handle->xrun[0] = false;
    }
    if ( stream_.mode != OUTPUT && handle->xrun[1] == true ) {
      status |= RTAUDIO_INPUT_OVERFLOW;
      handle->xrun[1] = false;
    }
    int cbReturnValue = callback( stream_.userBuffer[0], stream_.userBuffer[1],
                                  stream_.bufferSize, streamTime, status, info->userData );
    if ( cbReturnValue == 2 ) {
      stream_.state = STREAM_STOPPING;
      handle->drainCounter = 2;
      abortStream();
      return;
    }
    else if ( cbReturnValue == 1 ) {
      handle->drainCounter = 1;
      handle->internalDrain = true;
    }
  }

  HRESULT result;
  DWORD currentWritePointer, safeWritePointer;
  DWORD currentReadPointer, safeReadPointer;
  UINT nextWritePointer;

  LPVOID buffer1 = NULL;
  LPVOID buffer2 = NULL;
  DWORD bufferSize1 = 0;
  DWORD bufferSize2 = 0;

  char *buffer;
  long bufferBytes;

  MUTEX_LOCK( &stream_.mutex );
  if ( stream_.state == STREAM_STOPPED ) {
    MUTEX_UNLOCK( &stream_.mutex );
    return;
  }

  if ( buffersRolling == false ) {
    if ( stream_.mode == DUPLEX ) {
      //assert( handle->dsBufferSize[0] == handle->dsBufferSize[1] );

      // It takes a while for the devices to get rolling. As a result,
      // there's no guarantee that the capture and write device pointers
      // will move in lockstep.  Wait here for both devices to start
      // rolling, and then set our buffer pointers accordingly.
      // e.g. Crystal Drivers: the capture buffer starts up 5700 to 9600
      // bytes later than the write buffer.

      // Stub: a serious risk of having a pre-emptive scheduling round
      // take place between the two GetCurrentPosition calls... but I'm
      // really not sure how to solve the problem.  Temporarily boost to
      // Realtime priority, maybe; but I'm not sure what priority the
      // DirectSound service threads run at. We *should* be roughly
      // within a ms or so of correct.

      LPDIRECTSOUNDBUFFER dsWriteBuffer = (LPDIRECTSOUNDBUFFER) handle->buffer[0];
      LPDIRECTSOUNDCAPTUREBUFFER dsCaptureBuffer = (LPDIRECTSOUNDCAPTUREBUFFER) handle->buffer[1];

      DWORD startSafeWritePointer, startSafeReadPointer;

      result = dsWriteBuffer->GetCurrentPosition( NULL, &startSafeWritePointer );
      if ( FAILED( result ) ) {
        errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current write position!";
        errorText_ = errorStream_.str();
        MUTEX_UNLOCK( &stream_.mutex );
        error( RtAudioError::SYSTEM_ERROR );
        return;
      }
      result = dsCaptureBuffer->GetCurrentPosition( NULL, &startSafeReadPointer );
      if ( FAILED( result ) ) {
        errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current read position!";
        errorText_ = errorStream_.str();
        MUTEX_UNLOCK( &stream_.mutex );
        error( RtAudioError::SYSTEM_ERROR );
        return;
      }
      while ( true ) {
        result = dsWriteBuffer->GetCurrentPosition( NULL, &safeWritePointer );
        if ( FAILED( result ) ) {
          errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current write position!";
          errorText_ = errorStream_.str();
          MUTEX_UNLOCK( &stream_.mutex );
          error( RtAudioError::SYSTEM_ERROR );
          return;
        }
        result = dsCaptureBuffer->GetCurrentPosition( NULL, &safeReadPointer );
        if ( FAILED( result ) ) {
          errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current read position!";
          errorText_ = errorStream_.str();
          MUTEX_UNLOCK( &stream_.mutex );
          error( RtAudioError::SYSTEM_ERROR );
          return;
        }
        if ( safeWritePointer != startSafeWritePointer && safeReadPointer != startSafeReadPointer ) break;
        Sleep( 1 );
      }

      //assert( handle->dsBufferSize[0] == handle->dsBufferSize[1] );

      handle->bufferPointer[0] = safeWritePointer + handle->dsPointerLeadTime[0];
      if ( handle->bufferPointer[0] >= handle->dsBufferSize[0] ) handle->bufferPointer[0] -= handle->dsBufferSize[0];
      handle->bufferPointer[1] = safeReadPointer;
    }
    else if ( stream_.mode == OUTPUT ) {

      // Set the proper nextWritePosition after initial startup.
      LPDIRECTSOUNDBUFFER dsWriteBuffer = (LPDIRECTSOUNDBUFFER) handle->buffer[0];
      result = dsWriteBuffer->GetCurrentPosition( &currentWritePointer, &safeWritePointer );
      if ( FAILED( result ) ) {
        errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current write position!";
        errorText_ = errorStream_.str();
        MUTEX_UNLOCK( &stream_.mutex );
        error( RtAudioError::SYSTEM_ERROR );
        return;
      }
      handle->bufferPointer[0] = safeWritePointer + handle->dsPointerLeadTime[0];
      if ( handle->bufferPointer[0] >= handle->dsBufferSize[0] ) handle->bufferPointer[0] -= handle->dsBufferSize[0];
    }

    buffersRolling = true;
  }

  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    
    LPDIRECTSOUNDBUFFER dsBuffer = (LPDIRECTSOUNDBUFFER) handle->buffer[0];

    if ( handle->drainCounter > 1 ) { // write zeros to the output stream
      bufferBytes = stream_.bufferSize * stream_.nUserChannels[0];
      bufferBytes *= formatBytes( stream_.userFormat );
      memset( stream_.userBuffer[0], 0, bufferBytes );
    }

    // Setup parameters and do buffer conversion if necessary.
    if ( stream_.doConvertBuffer[0] ) {
      buffer = stream_.deviceBuffer;
      convertBuffer( buffer, stream_.userBuffer[0], stream_.convertInfo[0] );
      bufferBytes = stream_.bufferSize * stream_.nDeviceChannels[0];
      bufferBytes *= formatBytes( stream_.deviceFormat[0] );
    }
    else {
      buffer = stream_.userBuffer[0];
      bufferBytes = stream_.bufferSize * stream_.nUserChannels[0];
      bufferBytes *= formatBytes( stream_.userFormat );
    }

    // No byte swapping necessary in DirectSound implementation.

    // Ahhh ... windoze.  16-bit data is signed but 8-bit data is
    // unsigned.  So, we need to convert our signed 8-bit data here to
    // unsigned.
    if ( stream_.deviceFormat[0] == RTAUDIO_SINT8 )
      for ( int i=0; i<bufferBytes; i++ ) buffer[i] = (unsigned char) ( buffer[i] + 128 );

    DWORD dsBufferSize = handle->dsBufferSize[0];
    nextWritePointer = handle->bufferPointer[0];

    DWORD endWrite, leadPointer;
    while ( true ) {
      // Find out where the read and "safe write" pointers are.
      result = dsBuffer->GetCurrentPosition( &currentWritePointer, &safeWritePointer );
      if ( FAILED( result ) ) {
        errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current write position!";
        errorText_ = errorStream_.str();
        MUTEX_UNLOCK( &stream_.mutex );
        error( RtAudioError::SYSTEM_ERROR );
        return;
      }

      // We will copy our output buffer into the region between
      // safeWritePointer and leadPointer.  If leadPointer is not
      // beyond the next endWrite position, wait until it is.
      leadPointer = safeWritePointer + handle->dsPointerLeadTime[0];
      //std::cout << "safeWritePointer = " << safeWritePointer << ", leadPointer = " << leadPointer << ", nextWritePointer = " << nextWritePointer << std::endl;
      if ( leadPointer > dsBufferSize ) leadPointer -= dsBufferSize;
      if ( leadPointer < nextWritePointer ) leadPointer += dsBufferSize; // unwrap offset
      endWrite = nextWritePointer + bufferBytes;

      // Check whether the entire write region is behind the play pointer.
      if ( leadPointer >= endWrite ) break;

      // If we are here, then we must wait until the leadPointer advances
      // beyond the end of our next write region. We use the
      // Sleep() function to suspend operation until that happens.
      double millis = ( endWrite - leadPointer ) * 1000.0;
      millis /= ( formatBytes( stream_.deviceFormat[0]) * stream_.nDeviceChannels[0] * stream_.sampleRate);
      if ( millis < 1.0 ) millis = 1.0;
      Sleep( (DWORD) millis );
    }

    if ( dsPointerBetween( nextWritePointer, safeWritePointer, currentWritePointer, dsBufferSize )
         || dsPointerBetween( endWrite, safeWritePointer, currentWritePointer, dsBufferSize ) ) { 
      // We've strayed into the forbidden zone ... resync the read pointer.
      handle->xrun[0] = true;
      nextWritePointer = safeWritePointer + handle->dsPointerLeadTime[0] - bufferBytes;
      if ( nextWritePointer >= dsBufferSize ) nextWritePointer -= dsBufferSize;
      handle->bufferPointer[0] = nextWritePointer;
      endWrite = nextWritePointer + bufferBytes;
    }

    // Lock free space in the buffer
    result = dsBuffer->Lock( nextWritePointer, bufferBytes, &buffer1,
                             &bufferSize1, &buffer2, &bufferSize2, 0 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") locking buffer during playback!";
      errorText_ = errorStream_.str();
      MUTEX_UNLOCK( &stream_.mutex );
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }

    // Copy our buffer into the DS buffer
    CopyMemory( buffer1, buffer, bufferSize1 );
    if ( buffer2 != NULL ) CopyMemory( buffer2, buffer+bufferSize1, bufferSize2 );

    // Update our buffer offset and unlock sound buffer
    dsBuffer->Unlock( buffer1, bufferSize1, buffer2, bufferSize2 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") unlocking buffer during playback!";
      errorText_ = errorStream_.str();
      MUTEX_UNLOCK( &stream_.mutex );
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }
    nextWritePointer = ( nextWritePointer + bufferSize1 + bufferSize2 ) % dsBufferSize;
    handle->bufferPointer[0] = nextWritePointer;
  }

  // Don't bother draining input
  if ( handle->drainCounter ) {
    handle->drainCounter++;
    goto unlock;
  }

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX ) {

    // Setup parameters.
    if ( stream_.doConvertBuffer[1] ) {
      buffer = stream_.deviceBuffer;
      bufferBytes = stream_.bufferSize * stream_.nDeviceChannels[1];
      bufferBytes *= formatBytes( stream_.deviceFormat[1] );
    }
    else {
      buffer = stream_.userBuffer[1];
      bufferBytes = stream_.bufferSize * stream_.nUserChannels[1];
      bufferBytes *= formatBytes( stream_.userFormat );
    }

    LPDIRECTSOUNDCAPTUREBUFFER dsBuffer = (LPDIRECTSOUNDCAPTUREBUFFER) handle->buffer[1];
    long nextReadPointer = handle->bufferPointer[1];
    DWORD dsBufferSize = handle->dsBufferSize[1];

    // Find out where the write and "safe read" pointers are.
    result = dsBuffer->GetCurrentPosition( &currentReadPointer, &safeReadPointer );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current read position!";
      errorText_ = errorStream_.str();
      MUTEX_UNLOCK( &stream_.mutex );
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }

    if ( safeReadPointer < (DWORD)nextReadPointer ) safeReadPointer += dsBufferSize; // unwrap offset
    DWORD endRead = nextReadPointer + bufferBytes;

    // Handling depends on whether we are INPUT or DUPLEX. 
    // If we're in INPUT mode then waiting is a good thing. If we're in DUPLEX mode,
    // then a wait here will drag the write pointers into the forbidden zone.
    // 
    // In DUPLEX mode, rather than wait, we will back off the read pointer until 
    // it's in a safe position. This causes dropouts, but it seems to be the only 
    // practical way to sync up the read and write pointers reliably, given the 
    // the very complex relationship between phase and increment of the read and write 
    // pointers.
    //
    // In order to minimize audible dropouts in DUPLEX mode, we will
    // provide a pre-roll period of 0.5 seconds in which we return
    // zeros from the read buffer while the pointers sync up.

    if ( stream_.mode == DUPLEX ) {
      if ( safeReadPointer < endRead ) {
        if ( duplexPrerollBytes <= 0 ) {
          // Pre-roll time over. Be more agressive.
          int adjustment = endRead-safeReadPointer;

          handle->xrun[1] = true;
          // Two cases:
          //   - large adjustments: we've probably run out of CPU cycles, so just resync exactly,
          //     and perform fine adjustments later.
          //   - small adjustments: back off by twice as much.
          if ( adjustment >= 2*bufferBytes )
            nextReadPointer = safeReadPointer-2*bufferBytes;
          else
            nextReadPointer = safeReadPointer-bufferBytes-adjustment;

          if ( nextReadPointer < 0 ) nextReadPointer += dsBufferSize;

        }
        else {
          // In pre=roll time. Just do it.
          nextReadPointer = safeReadPointer - bufferBytes;
          while ( nextReadPointer < 0 ) nextReadPointer += dsBufferSize;
        }
        endRead = nextReadPointer + bufferBytes;
      }
    }
    else { // mode == INPUT
      while ( safeReadPointer < endRead && stream_.callbackInfo.isRunning ) {
        // See comments for playback.
        double millis = (endRead - safeReadPointer) * 1000.0;
        millis /= ( formatBytes(stream_.deviceFormat[1]) * stream_.nDeviceChannels[1] * stream_.sampleRate);
        if ( millis < 1.0 ) millis = 1.0;
        Sleep( (DWORD) millis );

        // Wake up and find out where we are now.
        result = dsBuffer->GetCurrentPosition( &currentReadPointer, &safeReadPointer );
        if ( FAILED( result ) ) {
          errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") getting current read position!";
          errorText_ = errorStream_.str();
          MUTEX_UNLOCK( &stream_.mutex );
          error( RtAudioError::SYSTEM_ERROR );
          return;
        }
      
        if ( safeReadPointer < (DWORD)nextReadPointer ) safeReadPointer += dsBufferSize; // unwrap offset
      }
    }

    // Lock free space in the buffer
    result = dsBuffer->Lock( nextReadPointer, bufferBytes, &buffer1,
                             &bufferSize1, &buffer2, &bufferSize2, 0 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") locking capture buffer!";
      errorText_ = errorStream_.str();
      MUTEX_UNLOCK( &stream_.mutex );
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }

    if ( duplexPrerollBytes <= 0 ) {
      // Copy our buffer into the DS buffer
      CopyMemory( buffer, buffer1, bufferSize1 );
      if ( buffer2 != NULL ) CopyMemory( buffer+bufferSize1, buffer2, bufferSize2 );
    }
    else {
      memset( buffer, 0, bufferSize1 );
      if ( buffer2 != NULL ) memset( buffer + bufferSize1, 0, bufferSize2 );
      duplexPrerollBytes -= bufferSize1 + bufferSize2;
    }

    // Update our buffer offset and unlock sound buffer
    nextReadPointer = ( nextReadPointer + bufferSize1 + bufferSize2 ) % dsBufferSize;
    dsBuffer->Unlock( buffer1, bufferSize1, buffer2, bufferSize2 );
    if ( FAILED( result ) ) {
      errorStream_ << "RtApiDs::callbackEvent: error (" << getErrorString( result ) << ") unlocking capture buffer!";
      errorText_ = errorStream_.str();
      MUTEX_UNLOCK( &stream_.mutex );
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }
    handle->bufferPointer[1] = nextReadPointer;

    // No byte swapping necessary in DirectSound implementation.

    // If necessary, convert 8-bit data from unsigned to signed.
    if ( stream_.deviceFormat[1] == RTAUDIO_SINT8 )
      for ( int j=0; j<bufferBytes; j++ ) buffer[j] = (signed char) ( buffer[j] - 128 );

    // Do buffer conversion if necessary.
    if ( stream_.doConvertBuffer[1] )
      convertBuffer( stream_.userBuffer[1], stream_.deviceBuffer, stream_.convertInfo[1] );
  }

 unlock:
  MUTEX_UNLOCK( &stream_.mutex );
  RtApi::tickStreamTime();
}
#pragma warning ( pop )

// Definitions for utility functions and callbacks
// specific to the DirectSound implementation.

static unsigned __stdcall callbackHandler( void *ptr )
{
  CallbackInfo *info = (CallbackInfo *) ptr;
  RtApiDs *object = (RtApiDs *) info->object;
  bool* isRunning = &info->isRunning;

  while ( *isRunning == true ) {
    object->callbackEvent();
  }

  _endthreadex( 0 );
  return 0;
}

static BOOL CALLBACK deviceQueryCallback( LPGUID lpguid,
                                          LPCTSTR description,
                                          LPCTSTR /*module*/,
                                          LPVOID lpContext )
{
  struct DsProbeData& probeInfo = *(struct DsProbeData*) lpContext;
  std::vector<struct DsDevice>& dsDevices = *probeInfo.dsDevices;

  HRESULT hr;
  bool validDevice = false;
  if ( probeInfo.isInput == true ) {
    DSCCAPS caps;
    LPDIRECTSOUNDCAPTURE object;

    hr = DirectSoundCaptureCreate(  lpguid, &object,   NULL );
    if ( hr != DS_OK ) return TRUE;

    caps.dwSize = sizeof(caps);
    hr = object->GetCaps( &caps );
    if ( hr == DS_OK ) {
      if ( caps.dwChannels > 0 && caps.dwFormats > 0 )
        validDevice = true;
    }
    object->Release();
  }
  else {
    DSCAPS caps;
    LPDIRECTSOUND object;
    hr = DirectSoundCreate(  lpguid, &object,   NULL );
    if ( hr != DS_OK ) return TRUE;

    caps.dwSize = sizeof(caps);
    hr = object->GetCaps( &caps );
    if ( hr == DS_OK ) {
      if ( caps.dwFlags & DSCAPS_PRIMARYMONO || caps.dwFlags & DSCAPS_PRIMARYSTEREO )
        validDevice = true;
    }
    object->Release();
  }

  // If good device, then save its name and guid.
  std::string name = convertCharPointerToStdString( description );
  //if ( name == "Primary Sound Driver" || name == "Primary Sound Capture Driver" )
  if ( lpguid == NULL )
    name = "Default Device";
  if ( validDevice ) {
    for ( unsigned int i=0; i<dsDevices.size(); i++ ) {
      if ( dsDevices[i].name == name ) {
        dsDevices[i].found = true;
        if ( probeInfo.isInput ) {
          dsDevices[i].id[1] = lpguid;
          dsDevices[i].validId[1] = true;
        }
        else {
          dsDevices[i].id[0] = lpguid;
          dsDevices[i].validId[0] = true;
        }
        return TRUE;
      }
    }

    DsDevice device;
    device.name = name;
    device.found = true;
    if ( probeInfo.isInput ) {
      device.id[1] = lpguid;
      device.validId[1] = true;
    }
    else {
      device.id[0] = lpguid;
      device.validId[0] = true;
    }
    dsDevices.push_back( device );
  }

  return TRUE;
}

static const char* getErrorString( int code )
{
  switch ( code ) {

  case DSERR_ALLOCATED:
    return "Already allocated";

  case DSERR_CONTROLUNAVAIL:
    return "Control unavailable";

  case DSERR_INVALIDPARAM:
    return "Invalid parameter";

  case DSERR_INVALIDCALL:
    return "Invalid call";

  case DSERR_GENERIC:
    return "Generic error";

  case DSERR_PRIOLEVELNEEDED:
    return "Priority level needed";

  case DSERR_OUTOFMEMORY:
    return "Out of memory";

  case DSERR_BADFORMAT:
    return "The sample rate or the channel format is not supported";

  case DSERR_UNSUPPORTED:
    return "Not supported";

  case DSERR_NODRIVER:
    return "No driver";

  case DSERR_ALREADYINITIALIZED:
    return "Already initialized";

  case DSERR_NOAGGREGATION:
    return "No aggregation";

  case DSERR_BUFFERLOST:
    return "Buffer lost";

  case DSERR_OTHERAPPHASPRIO:
    return "Another application already has priority";

  case DSERR_UNINITIALIZED:
    return "Uninitialized";

  default:
    return "DirectSound unknown error";
  }
}

#include "Windows/HideWindowsPlatformTypes.h"

//******************** End of __WINDOWS_DS__ *********************//
#endif


#if defined(__LINUX_ALSA__)

#include <alsa/asoundlib.h>
#include <unistd.h>

  // A structure to hold various information related to the ALSA API
  // implementation.
struct AlsaHandle {
  snd_pcm_t *handles[2];
  bool synchronized;
  bool xrun[2];
  pthread_cond_t runnable_cv;
  bool runnable;

  AlsaHandle()
    :synchronized(false), runnable(false) { xrun[0] = false; xrun[1] = false; }
};

static void *alsaCallbackHandler( void * ptr );

RtApiAlsa :: RtApiAlsa()
{
  // Nothing to do here.
}

RtApiAlsa :: ~RtApiAlsa()
{
  if ( stream_.state != STREAM_CLOSED ) closeStream();
}

unsigned int RtApiAlsa :: getDeviceCount( void )
{
  unsigned nDevices = 0;
  int result, subdevice, card;
  char name[64];
  snd_ctl_t *handle;

  // Count cards and devices
  card = -1;
  snd_card_next( &card );
  while ( card >= 0 ) {
    sprintf( name, "hw:%d", card );
    result = snd_ctl_open( &handle, name, 0 );
    if ( result < 0 ) {
      errorStream_ << "RtApiAlsa::getDeviceCount: control open, card = " << card << ", " << snd_strerror( result ) << ".";
      errorText_ = errorStream_.str();
      error( RtAudioError::WARNING );
      goto nextcard;
    }
    subdevice = -1;
    while( 1 ) {
      result = snd_ctl_pcm_next_device( handle, &subdevice );
      if ( result < 0 ) {
        errorStream_ << "RtApiAlsa::getDeviceCount: control next device, card = " << card << ", " << snd_strerror( result ) << ".";
        errorText_ = errorStream_.str();
        error( RtAudioError::WARNING );
        break;
      }
      if ( subdevice < 0 )
        break;
      nDevices++;
    }
  nextcard:
    snd_ctl_close( handle );
    snd_card_next( &card );
  }

  result = snd_ctl_open( &handle, "default", 0 );
  if (result == 0) {
    nDevices++;
    snd_ctl_close( handle );
  }

  return nDevices;
}

RtAudio::DeviceInfo RtApiAlsa :: getDeviceInfo( unsigned int device )
{
  RtAudio::DeviceInfo info;
  info.probed = false;

  unsigned nDevices = 0;
  int result, subdevice, card;
  char name[64];
  snd_ctl_t *chandle;

  // Count cards and devices
  card = -1;
  subdevice = -1;
  snd_card_next( &card );
  while ( card >= 0 ) {
    sprintf( name, "hw:%d", card );
    result = snd_ctl_open( &chandle, name, SND_CTL_NONBLOCK );
    if ( result < 0 ) {
      errorStream_ << "RtApiAlsa::getDeviceInfo: control open, card = " << card << ", " << snd_strerror( result ) << ".";
      errorText_ = errorStream_.str();
      error( RtAudioError::WARNING );
      goto nextcard;
    }
    subdevice = -1;
    while( 1 ) {
      result = snd_ctl_pcm_next_device( chandle, &subdevice );
      if ( result < 0 ) {
        errorStream_ << "RtApiAlsa::getDeviceInfo: control next device, card = " << card << ", " << snd_strerror( result ) << ".";
        errorText_ = errorStream_.str();
        error( RtAudioError::WARNING );
        break;
      }
      if ( subdevice < 0 ) break;
      if ( nDevices == device ) {
        sprintf( name, "hw:%d,%d", card, subdevice );
        goto foundDevice;
      }
      nDevices++;
    }
  nextcard:
    snd_ctl_close( chandle );
    snd_card_next( &card );
  }

  result = snd_ctl_open( &chandle, "default", SND_CTL_NONBLOCK );
  if ( result == 0 ) {
    if ( nDevices == device ) {
      strcpy( name, "default" );
      goto foundDevice;
    }
    nDevices++;
  }

  if ( nDevices == 0 ) {
    errorText_ = "RtApiAlsa::getDeviceInfo: no devices found!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  if ( device >= nDevices ) {
    errorText_ = "RtApiAlsa::getDeviceInfo: device ID is invalid!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

 foundDevice:

  // If a stream is already open, we cannot probe the stream devices.
  // Thus, use the saved results.
  if ( stream_.state != STREAM_CLOSED &&
       ( stream_.device[0] == device || stream_.device[1] == device ) ) {
    snd_ctl_close( chandle );
    if ( device >= devices_.size() ) {
      errorText_ = "RtApiAlsa::getDeviceInfo: device ID was not present before stream was opened.";
      error( RtAudioError::WARNING );
      return info;
    }
    return devices_[ device ];
  }

  int openMode = SND_PCM_ASYNC;
  snd_pcm_stream_t stream;
  snd_pcm_info_t *pcminfo;
  snd_pcm_info_alloca( &pcminfo );
  snd_pcm_t *phandle;
  snd_pcm_hw_params_t *params;
  snd_pcm_hw_params_alloca( &params );

  // First try for playback unless default device (which has subdev -1)
  stream = SND_PCM_STREAM_PLAYBACK;
  snd_pcm_info_set_stream( pcminfo, stream );
  if ( subdevice != -1 ) {
    snd_pcm_info_set_device( pcminfo, subdevice );
    snd_pcm_info_set_subdevice( pcminfo, 0 );

    result = snd_ctl_pcm_info( chandle, pcminfo );
    if ( result < 0 ) {
      // Device probably doesn't support playback.
      goto captureProbe;
    }
  }

  result = snd_pcm_open( &phandle, name, stream, openMode | SND_PCM_NONBLOCK );
  if ( result < 0 ) {
    errorStream_ << "RtApiAlsa::getDeviceInfo: snd_pcm_open error for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    goto captureProbe;
  }

  // The device is open ... fill the parameter structure.
  result = snd_pcm_hw_params_any( phandle, params );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::getDeviceInfo: snd_pcm_hw_params error for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    goto captureProbe;
  }

  // Get output channel information.
  unsigned int value;
  result = snd_pcm_hw_params_get_channels_max( params, &value );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::getDeviceInfo: error getting device (" << name << ") output channels, " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    goto captureProbe;
  }
  info.outputChannels = value;
  snd_pcm_close( phandle );

 captureProbe:
  stream = SND_PCM_STREAM_CAPTURE;
  snd_pcm_info_set_stream( pcminfo, stream );

  // Now try for capture unless default device (with subdev = -1)
  if ( subdevice != -1 ) {
    result = snd_ctl_pcm_info( chandle, pcminfo );
    snd_ctl_close( chandle );
    if ( result < 0 ) {
      // Device probably doesn't support capture.
      if ( info.outputChannels == 0 ) return info;
      goto probeParameters;
    }
  }
  else
    snd_ctl_close( chandle );

  result = snd_pcm_open( &phandle, name, stream, openMode | SND_PCM_NONBLOCK);
  if ( result < 0 ) {
    errorStream_ << "RtApiAlsa::getDeviceInfo: snd_pcm_open error for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    if ( info.outputChannels == 0 ) return info;
    goto probeParameters;
  }

  // The device is open ... fill the parameter structure.
  result = snd_pcm_hw_params_any( phandle, params );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::getDeviceInfo: snd_pcm_hw_params error for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    if ( info.outputChannels == 0 ) return info;
    goto probeParameters;
  }

  result = snd_pcm_hw_params_get_channels_max( params, &value );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::getDeviceInfo: error getting device (" << name << ") input channels, " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    if ( info.outputChannels == 0 ) return info;
    goto probeParameters;
  }
  info.inputChannels = value;
  snd_pcm_close( phandle );

  // If device opens for both playback and capture, we determine the channels.
  if ( info.outputChannels > 0 && info.inputChannels > 0 )
    info.duplexChannels = (info.outputChannels > info.inputChannels) ? info.inputChannels : info.outputChannels;

  // ALSA doesn't provide default devices so we'll use the first available one.
  if ( device == 0 && info.outputChannels > 0 )
    info.isDefaultOutput = true;
  if ( device == 0 && info.inputChannels > 0 )
    info.isDefaultInput = true;

 probeParameters:
  // At this point, we just need to figure out the supported data
  // formats and sample rates.  We'll proceed by opening the device in
  // the direction with the maximum number of channels, or playback if
  // they are equal.  This might limit our sample rate options, but so
  // be it.

  if ( info.outputChannels >= info.inputChannels )
    stream = SND_PCM_STREAM_PLAYBACK;
  else
    stream = SND_PCM_STREAM_CAPTURE;
  snd_pcm_info_set_stream( pcminfo, stream );

  result = snd_pcm_open( &phandle, name, stream, openMode | SND_PCM_NONBLOCK);
  if ( result < 0 ) {
    errorStream_ << "RtApiAlsa::getDeviceInfo: snd_pcm_open error for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // The device is open ... fill the parameter structure.
  result = snd_pcm_hw_params_any( phandle, params );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::getDeviceInfo: snd_pcm_hw_params error for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Test our discrete set of sample rate values.
  info.sampleRates.clear();
  for ( unsigned int i=0; i<MAX_SAMPLE_RATES; i++ ) {
    if ( snd_pcm_hw_params_test_rate( phandle, params, SAMPLE_RATES[i], 0 ) == 0 ) {
      info.sampleRates.push_back( SAMPLE_RATES[i] );

      if ( !info.preferredSampleRate || ( SAMPLE_RATES[i] <= 48000 && SAMPLE_RATES[i] > info.preferredSampleRate ) )
        info.preferredSampleRate = SAMPLE_RATES[i];
    }
  }
  if ( info.sampleRates.size() == 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::getDeviceInfo: no supported sample rates found for device (" << name << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Probe the supported data formats ... we don't care about endian-ness just yet
  snd_pcm_format_t format;
  info.nativeFormats = 0;
  format = SND_PCM_FORMAT_S8;
  if ( snd_pcm_hw_params_test_format( phandle, params, format ) == 0 )
    info.nativeFormats |= RTAUDIO_SINT8;
  format = SND_PCM_FORMAT_S16;
  if ( snd_pcm_hw_params_test_format( phandle, params, format ) == 0 )
    info.nativeFormats |= RTAUDIO_SINT16;
  format = SND_PCM_FORMAT_S24;
  if ( snd_pcm_hw_params_test_format( phandle, params, format ) == 0 )
    info.nativeFormats |= RTAUDIO_SINT24;
  format = SND_PCM_FORMAT_S32;
  if ( snd_pcm_hw_params_test_format( phandle, params, format ) == 0 )
    info.nativeFormats |= RTAUDIO_SINT32;
  format = SND_PCM_FORMAT_FLOAT;
  if ( snd_pcm_hw_params_test_format( phandle, params, format ) == 0 )
    info.nativeFormats |= RTAUDIO_FLOAT32;
  format = SND_PCM_FORMAT_FLOAT64;
  if ( snd_pcm_hw_params_test_format( phandle, params, format ) == 0 )
    info.nativeFormats |= RTAUDIO_FLOAT64;

  // Check that we have at least one supported format
  if ( info.nativeFormats == 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::getDeviceInfo: pcm device (" << name << ") data format not supported by RtAudio.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Get the device name
  char *cardname;
  result = snd_card_get_name( card, &cardname );
  if ( result >= 0 ) {
    sprintf( name, "hw:%s,%d", cardname, subdevice );
    free( cardname );
  }
  info.name = name;

  // That's all ... close the device and return
  snd_pcm_close( phandle );
  info.probed = true;
  return info;
}

void RtApiAlsa :: saveDeviceInfo( void )
{
  devices_.clear();

  unsigned int nDevices = getDeviceCount();
  devices_.resize( nDevices );
  for ( unsigned int i=0; i<nDevices; i++ )
    devices_[i] = getDeviceInfo( i );
}

bool RtApiAlsa :: probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                                   unsigned int firstChannel, unsigned int sampleRate,
                                   RtAudioFormat format, unsigned int *bufferSize,
                                   RtAudio::StreamOptions *options )

{
#if defined(__RTAUDIO_DEBUG__)
  snd_output_t *out;
  snd_output_stdio_attach(&out, stderr, 0);
#endif

  // I'm not using the "plug" interface ... too much inconsistent behavior.

  unsigned nDevices = 0;
  int result, subdevice, card;
  char name[64];
  snd_ctl_t *chandle;

  if ( options && options->flags & RTAUDIO_ALSA_USE_DEFAULT )
    snprintf(name, sizeof(name), "%s", "default");
  else {
    // Count cards and devices
    card = -1;
    snd_card_next( &card );
    while ( card >= 0 ) {
      sprintf( name, "hw:%d", card );
      result = snd_ctl_open( &chandle, name, SND_CTL_NONBLOCK );
      if ( result < 0 ) {
        errorStream_ << "RtApiAlsa::probeDeviceOpen: control open, card = " << card << ", " << snd_strerror( result ) << ".";
        errorText_ = errorStream_.str();
        return FAILURE;
      }
      subdevice = -1;
      while( 1 ) {
        result = snd_ctl_pcm_next_device( chandle, &subdevice );
        if ( result < 0 ) break;
        if ( subdevice < 0 ) break;
        if ( nDevices == device ) {
          sprintf( name, "hw:%d,%d", card, subdevice );
          snd_ctl_close( chandle );
          goto foundDevice;
        }
        nDevices++;
      }
      snd_ctl_close( chandle );
      snd_card_next( &card );
    }

    result = snd_ctl_open( &chandle, "default", SND_CTL_NONBLOCK );
    if ( result == 0 ) {
      if ( nDevices == device ) {
        strcpy( name, "default" );
        goto foundDevice;
      }
      nDevices++;
    }

    if ( nDevices == 0 ) {
      // This should not happen because a check is made before this function is called.
      errorText_ = "RtApiAlsa::probeDeviceOpen: no devices found!";
      return FAILURE;
    }

    if ( device >= nDevices ) {
      // This should not happen because a check is made before this function is called.
      errorText_ = "RtApiAlsa::probeDeviceOpen: device ID is invalid!";
      return FAILURE;
    }
  }

 foundDevice:

  // The getDeviceInfo() function will not work for a device that is
  // already open.  Thus, we'll probe the system before opening a
  // stream and save the results for use by getDeviceInfo().
  if ( mode == OUTPUT || ( mode == INPUT && stream_.mode != OUTPUT ) ) // only do once
    this->saveDeviceInfo();

  snd_pcm_stream_t stream;
  if ( mode == OUTPUT )
    stream = SND_PCM_STREAM_PLAYBACK;
  else
    stream = SND_PCM_STREAM_CAPTURE;

  snd_pcm_t *phandle;
  int openMode = SND_PCM_ASYNC;
  result = snd_pcm_open( &phandle, name, stream, openMode );
  if ( result < 0 ) {
    if ( mode == OUTPUT )
      errorStream_ << "RtApiAlsa::probeDeviceOpen: pcm device (" << name << ") won't open for output.";
    else
      errorStream_ << "RtApiAlsa::probeDeviceOpen: pcm device (" << name << ") won't open for input.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Fill the parameter structure.
  snd_pcm_hw_params_t *hw_params;
  snd_pcm_hw_params_alloca( &hw_params );
  result = snd_pcm_hw_params_any( phandle, hw_params );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error getting pcm device (" << name << ") parameters, " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

#if defined(__RTAUDIO_DEBUG__)
  fprintf( stderr, "\nRtApiAlsa: dump hardware params just after device open:\n\n" );
  snd_pcm_hw_params_dump( hw_params, out );
#endif

  // Set access ... check user preference.
  if ( options && options->flags & RTAUDIO_NONINTERLEAVED ) {
    stream_.userInterleaved = false;
    result = snd_pcm_hw_params_set_access( phandle, hw_params, SND_PCM_ACCESS_RW_NONINTERLEAVED );
    if ( result < 0 ) {
      result = snd_pcm_hw_params_set_access( phandle, hw_params, SND_PCM_ACCESS_RW_INTERLEAVED );
      stream_.deviceInterleaved[mode] =  true;
    }
    else
      stream_.deviceInterleaved[mode] = false;
  }
  else {
    stream_.userInterleaved = true;
    result = snd_pcm_hw_params_set_access( phandle, hw_params, SND_PCM_ACCESS_RW_INTERLEAVED );
    if ( result < 0 ) {
      result = snd_pcm_hw_params_set_access( phandle, hw_params, SND_PCM_ACCESS_RW_NONINTERLEAVED );
      stream_.deviceInterleaved[mode] =  false;
    }
    else
      stream_.deviceInterleaved[mode] =  true;
  }

  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error setting pcm device (" << name << ") access, " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Determine how to set the device format.
  stream_.userFormat = format;
  snd_pcm_format_t deviceFormat = SND_PCM_FORMAT_UNKNOWN;

  if ( format == RTAUDIO_SINT8 )
    deviceFormat = SND_PCM_FORMAT_S8;
  else if ( format == RTAUDIO_SINT16 )
    deviceFormat = SND_PCM_FORMAT_S16;
  else if ( format == RTAUDIO_SINT24 )
    deviceFormat = SND_PCM_FORMAT_S24;
  else if ( format == RTAUDIO_SINT32 )
    deviceFormat = SND_PCM_FORMAT_S32;
  else if ( format == RTAUDIO_FLOAT32 )
    deviceFormat = SND_PCM_FORMAT_FLOAT;
  else if ( format == RTAUDIO_FLOAT64 )
    deviceFormat = SND_PCM_FORMAT_FLOAT64;

  if ( snd_pcm_hw_params_test_format(phandle, hw_params, deviceFormat) == 0) {
    stream_.deviceFormat[mode] = format;
    goto setFormat;
  }

  // The user requested format is not natively supported by the device.
  deviceFormat = SND_PCM_FORMAT_FLOAT64;
  if ( snd_pcm_hw_params_test_format( phandle, hw_params, deviceFormat ) == 0 ) {
    stream_.deviceFormat[mode] = RTAUDIO_FLOAT64;
    goto setFormat;
  }

  deviceFormat = SND_PCM_FORMAT_FLOAT;
  if ( snd_pcm_hw_params_test_format(phandle, hw_params, deviceFormat ) == 0 ) {
    stream_.deviceFormat[mode] = RTAUDIO_FLOAT32;
    goto setFormat;
  }

  deviceFormat = SND_PCM_FORMAT_S32;
  if ( snd_pcm_hw_params_test_format(phandle, hw_params, deviceFormat ) == 0 ) {
    stream_.deviceFormat[mode] = RTAUDIO_SINT32;
    goto setFormat;
  }

  deviceFormat = SND_PCM_FORMAT_S24;
  if ( snd_pcm_hw_params_test_format(phandle, hw_params, deviceFormat ) == 0 ) {
    stream_.deviceFormat[mode] = RTAUDIO_SINT24;
    goto setFormat;
  }

  deviceFormat = SND_PCM_FORMAT_S16;
  if ( snd_pcm_hw_params_test_format(phandle, hw_params, deviceFormat ) == 0 ) {
    stream_.deviceFormat[mode] = RTAUDIO_SINT16;
    goto setFormat;
  }

  deviceFormat = SND_PCM_FORMAT_S8;
  if ( snd_pcm_hw_params_test_format(phandle, hw_params, deviceFormat ) == 0 ) {
    stream_.deviceFormat[mode] = RTAUDIO_SINT8;
    goto setFormat;
  }

  // If we get here, no supported format was found.
  snd_pcm_close( phandle );
  errorStream_ << "RtApiAlsa::probeDeviceOpen: pcm device " << device << " data format not supported by RtAudio.";
  errorText_ = errorStream_.str();
  return FAILURE;

 setFormat:
  result = snd_pcm_hw_params_set_format( phandle, hw_params, deviceFormat );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error setting pcm device (" << name << ") data format, " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Determine whether byte-swaping is necessary.
  stream_.doByteSwap[mode] = false;
  if ( deviceFormat != SND_PCM_FORMAT_S8 ) {
    result = snd_pcm_format_cpu_endian( deviceFormat );
    if ( result == 0 )
      stream_.doByteSwap[mode] = true;
    else if (result < 0) {
      snd_pcm_close( phandle );
      errorStream_ << "RtApiAlsa::probeDeviceOpen: error getting pcm device (" << name << ") endian-ness, " << snd_strerror( result ) << ".";
      errorText_ = errorStream_.str();
      return FAILURE;
    }
  }

  // Set the sample rate.
  result = snd_pcm_hw_params_set_rate_near( phandle, hw_params, (unsigned int*) &sampleRate, 0 );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error setting sample rate on device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Determine the number of channels for this device.  We support a possible
  // minimum device channel number > than the value requested by the user.
  stream_.nUserChannels[mode] = channels;
  unsigned int value;
  result = snd_pcm_hw_params_get_channels_max( hw_params, &value );
  unsigned int deviceChannels = value;
  if ( result < 0 || deviceChannels < channels + firstChannel ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: requested channel parameters not supported by device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  result = snd_pcm_hw_params_get_channels_min( hw_params, &value );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error getting minimum channels for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }
  deviceChannels = value;
  if ( deviceChannels < channels + firstChannel ) deviceChannels = channels + firstChannel;
  stream_.nDeviceChannels[mode] = deviceChannels;

  // Set the device channels.
  result = snd_pcm_hw_params_set_channels( phandle, hw_params, deviceChannels );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error setting channels for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Set the buffer (or period) size.
  int dir = 0;
  snd_pcm_uframes_t periodSize = *bufferSize;
  result = snd_pcm_hw_params_set_period_size_near( phandle, hw_params, &periodSize, &dir );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error setting period size for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }
  *bufferSize = periodSize;

  // Set the buffer number, which in ALSA is referred to as the "period".
  unsigned int periods = 0;
  if ( options && options->flags & RTAUDIO_MINIMIZE_LATENCY ) periods = 2;
  if ( options && options->numberOfBuffers > 0 ) periods = options->numberOfBuffers;
  if ( periods < 2 ) periods = 4; // a fairly safe default value
  result = snd_pcm_hw_params_set_periods_near( phandle, hw_params, &periods, &dir );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error setting periods for device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // If attempting to setup a duplex stream, the bufferSize parameter
  // MUST be the same in both directions!
  if ( stream_.mode == OUTPUT && mode == INPUT && *bufferSize != stream_.bufferSize ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: system error setting buffer size for duplex stream on device (" << name << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  stream_.bufferSize = *bufferSize;

  // Install the hardware configuration
  result = snd_pcm_hw_params( phandle, hw_params );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error installing hardware configuration on device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

#if defined(__RTAUDIO_DEBUG__)
  fprintf(stderr, "\nRtApiAlsa: dump hardware params after installation:\n\n");
  snd_pcm_hw_params_dump( hw_params, out );
#endif

  // Set the software configuration to fill buffers with zeros and prevent device stopping on xruns.
  snd_pcm_sw_params_t *sw_params = NULL;
  snd_pcm_sw_params_alloca( &sw_params );
  snd_pcm_sw_params_current( phandle, sw_params );
  snd_pcm_sw_params_set_start_threshold( phandle, sw_params, *bufferSize );
  snd_pcm_sw_params_set_stop_threshold( phandle, sw_params, ULONG_MAX );
  snd_pcm_sw_params_set_silence_threshold( phandle, sw_params, 0 );

  // The following two settings were suggested by Theo Veenker
  //snd_pcm_sw_params_set_avail_min( phandle, sw_params, *bufferSize );
  //snd_pcm_sw_params_set_xfer_align( phandle, sw_params, 1 );

  // here are two options for a fix
  //snd_pcm_sw_params_set_silence_size( phandle, sw_params, ULONG_MAX );
  snd_pcm_uframes_t val;
  snd_pcm_sw_params_get_boundary( sw_params, &val );
  snd_pcm_sw_params_set_silence_size( phandle, sw_params, val );

  result = snd_pcm_sw_params( phandle, sw_params );
  if ( result < 0 ) {
    snd_pcm_close( phandle );
    errorStream_ << "RtApiAlsa::probeDeviceOpen: error installing software configuration on device (" << name << "), " << snd_strerror( result ) << ".";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

#if defined(__RTAUDIO_DEBUG__)
  fprintf(stderr, "\nRtApiAlsa: dump software params after installation:\n\n");
  snd_pcm_sw_params_dump( sw_params, out );
#endif

  // Set flags for buffer conversion
  stream_.doConvertBuffer[mode] = false;
  if ( stream_.userFormat != stream_.deviceFormat[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.nUserChannels[mode] < stream_.nDeviceChannels[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.userInterleaved != stream_.deviceInterleaved[mode] &&
       stream_.nUserChannels[mode] > 1 )
    stream_.doConvertBuffer[mode] = true;

  // Allocate the ApiHandle if necessary and then save.
  AlsaHandle *apiInfo = 0;
  if ( stream_.apiHandle == 0 ) {
    try {
      apiInfo = (AlsaHandle *) new AlsaHandle;
    }
    catch ( std::bad_alloc& ) {
      errorText_ = "RtApiAlsa::probeDeviceOpen: error allocating AlsaHandle memory.";
      goto error;
    }

    if ( pthread_cond_init( &apiInfo->runnable_cv, NULL ) ) {
      errorText_ = "RtApiAlsa::probeDeviceOpen: error initializing pthread condition variable.";
      goto error;
    }

    stream_.apiHandle = (void *) apiInfo;
    apiInfo->handles[0] = 0;
    apiInfo->handles[1] = 0;
  }
  else {
    apiInfo = (AlsaHandle *) stream_.apiHandle;
  }
  apiInfo->handles[mode] = phandle;
  phandle = 0;

  // Allocate necessary internal buffers.
  unsigned long bufferBytes;
  bufferBytes = stream_.nUserChannels[mode] * *bufferSize * formatBytes( stream_.userFormat );
  stream_.userBuffer[mode] = (char *) calloc( bufferBytes, 1 );
  if ( stream_.userBuffer[mode] == NULL ) {
    errorText_ = "RtApiAlsa::probeDeviceOpen: error allocating user buffer memory.";
    goto error;
  }

  if ( stream_.doConvertBuffer[mode] ) {

    bool makeBuffer = true;
    bufferBytes = stream_.nDeviceChannels[mode] * formatBytes( stream_.deviceFormat[mode] );
    if ( mode == INPUT ) {
      if ( stream_.mode == OUTPUT && stream_.deviceBuffer ) {
        unsigned long bytesOut = stream_.nDeviceChannels[0] * formatBytes( stream_.deviceFormat[0] );
        if ( bufferBytes <= bytesOut ) makeBuffer = false;
      }
    }

    if ( makeBuffer ) {
      bufferBytes *= *bufferSize;
      if ( stream_.deviceBuffer ) free( stream_.deviceBuffer );
      stream_.deviceBuffer = (char *) calloc( bufferBytes, 1 );
      if ( stream_.deviceBuffer == NULL ) {
        errorText_ = "RtApiAlsa::probeDeviceOpen: error allocating device buffer memory.";
        goto error;
      }
    }
  }

  stream_.sampleRate = sampleRate;
  stream_.nBuffers = periods;
  stream_.device[mode] = device;
  stream_.state = STREAM_STOPPED;

  // Setup the buffer conversion information structure.
  if ( stream_.doConvertBuffer[mode] ) setConvertInfo( mode, firstChannel );

  // Setup thread if necessary.
  if ( stream_.mode == OUTPUT && mode == INPUT ) {
    // We had already set up an output stream.
    stream_.mode = DUPLEX;
    // Link the streams if possible.
    apiInfo->synchronized = false;
    if ( snd_pcm_link( apiInfo->handles[0], apiInfo->handles[1] ) == 0 )
      apiInfo->synchronized = true;
    else {
      errorText_ = "RtApiAlsa::probeDeviceOpen: unable to synchronize input and output devices.";
      error( RtAudioError::WARNING );
    }
  }
  else {
    stream_.mode = mode;

    // Setup callback thread.
    stream_.callbackInfo.object = (void *) this;

    // Set the thread attributes for joinable and realtime scheduling
    // priority (optional).  The higher priority will only take affect
    // if the program is run as root or suid. Note, under Linux
    // processes with CAP_SYS_NICE privilege, a user can change
    // scheduling policy and priority (thus need not be root). See
    // POSIX "capabilities".
    pthread_attr_t attr;
    pthread_attr_init( &attr );
    pthread_attr_setdetachstate( &attr, PTHREAD_CREATE_JOINABLE );

#ifdef SCHED_RR // Undefined with some OSes (eg: NetBSD 1.6.x with GNU Pthread)
    if ( options && options->flags & RTAUDIO_SCHEDULE_REALTIME ) {
      // We previously attempted to increase the audio callback priority
      // to SCHED_RR here via the attributes.  However, while no errors
      // were reported in doing so, it did not work.  So, now this is
      // done in the alsaCallbackHandler function.
      stream_.callbackInfo.doRealtime = true;
      int priority = options->priority;
      int min = sched_get_priority_min( SCHED_RR );
      int max = sched_get_priority_max( SCHED_RR );
      if ( priority < min ) priority = min;
      else if ( priority > max ) priority = max;
      stream_.callbackInfo.priority = priority;
    }
#endif

    stream_.callbackInfo.isRunning = true;
    result = pthread_create( &stream_.callbackInfo.thread, &attr, alsaCallbackHandler, &stream_.callbackInfo );
    pthread_attr_destroy( &attr );
    if ( result ) {
      stream_.callbackInfo.isRunning = false;
      errorText_ = "RtApiAlsa::error creating callback thread!";
      goto error;
    }
  }

  return SUCCESS;

 error:
  if ( apiInfo ) {
    pthread_cond_destroy( &apiInfo->runnable_cv );
    if ( apiInfo->handles[0] ) snd_pcm_close( apiInfo->handles[0] );
    if ( apiInfo->handles[1] ) snd_pcm_close( apiInfo->handles[1] );
    delete apiInfo;
    stream_.apiHandle = 0;
  }

  if ( phandle) snd_pcm_close( phandle );

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.state = STREAM_CLOSED;
  return FAILURE;
}

void RtApiAlsa :: closeStream()
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiAlsa::closeStream(): no open stream to close!";
    error( RtAudioError::WARNING );
    return;
  }

  AlsaHandle *apiInfo = (AlsaHandle *) stream_.apiHandle;
  stream_.callbackInfo.isRunning = false;
  MUTEX_LOCK( &stream_.mutex );
  if ( stream_.state == STREAM_STOPPED ) {
    apiInfo->runnable = true;
    pthread_cond_signal( &apiInfo->runnable_cv );
  }
  MUTEX_UNLOCK( &stream_.mutex );
  pthread_join( stream_.callbackInfo.thread, NULL );

  if ( stream_.state == STREAM_RUNNING ) {
    stream_.state = STREAM_STOPPED;
    if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX )
      snd_pcm_drop( apiInfo->handles[0] );
    if ( stream_.mode == INPUT || stream_.mode == DUPLEX )
      snd_pcm_drop( apiInfo->handles[1] );
  }

  if ( apiInfo ) {
    pthread_cond_destroy( &apiInfo->runnable_cv );
    if ( apiInfo->handles[0] ) snd_pcm_close( apiInfo->handles[0] );
    if ( apiInfo->handles[1] ) snd_pcm_close( apiInfo->handles[1] );
    delete apiInfo;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.mode = UNINITIALIZED;
  stream_.state = STREAM_CLOSED;
}

void RtApiAlsa :: startStream()
{
  // This method calls snd_pcm_prepare if the device isn't already in that state.

  verifyStream();
  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiAlsa::startStream(): the stream is already running!";
    error( RtAudioError::WARNING );
    return;
  }

  MUTEX_LOCK( &stream_.mutex );

  int result = 0;
  snd_pcm_state_t state;
  AlsaHandle *apiInfo = (AlsaHandle *) stream_.apiHandle;
  snd_pcm_t **handle = (snd_pcm_t **) apiInfo->handles;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    state = snd_pcm_state( handle[0] );
    if ( state != SND_PCM_STATE_PREPARED ) {
      result = snd_pcm_prepare( handle[0] );
      if ( result < 0 ) {
        errorStream_ << "RtApiAlsa::startStream: error preparing output pcm device, " << snd_strerror( result ) << ".";
        errorText_ = errorStream_.str();
        goto unlock;
      }
    }
  }

  if ( ( stream_.mode == INPUT || stream_.mode == DUPLEX ) && !apiInfo->synchronized ) {
    result = snd_pcm_drop(handle[1]); // fix to remove stale data received since device has been open
    state = snd_pcm_state( handle[1] );
    if ( state != SND_PCM_STATE_PREPARED ) {
      result = snd_pcm_prepare( handle[1] );
      if ( result < 0 ) {
        errorStream_ << "RtApiAlsa::startStream: error preparing input pcm device, " << snd_strerror( result ) << ".";
        errorText_ = errorStream_.str();
        goto unlock;
      }
    }
  }

  stream_.state = STREAM_RUNNING;

 unlock:
  apiInfo->runnable = true;
  pthread_cond_signal( &apiInfo->runnable_cv );
  MUTEX_UNLOCK( &stream_.mutex );

  if ( result >= 0 ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiAlsa :: stopStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiAlsa::stopStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  stream_.state = STREAM_STOPPED;
  MUTEX_LOCK( &stream_.mutex );

  int result = 0;
  AlsaHandle *apiInfo = (AlsaHandle *) stream_.apiHandle;
  snd_pcm_t **handle = (snd_pcm_t **) apiInfo->handles;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    if ( apiInfo->synchronized ) 
      result = snd_pcm_drop( handle[0] );
    else
      result = snd_pcm_drain( handle[0] );
    if ( result < 0 ) {
      errorStream_ << "RtApiAlsa::stopStream: error draining output pcm device, " << snd_strerror( result ) << ".";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  if ( ( stream_.mode == INPUT || stream_.mode == DUPLEX ) && !apiInfo->synchronized ) {
    result = snd_pcm_drop( handle[1] );
    if ( result < 0 ) {
      errorStream_ << "RtApiAlsa::stopStream: error stopping input pcm device, " << snd_strerror( result ) << ".";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

 unlock:
  apiInfo->runnable = false; // fixes high CPU usage when stopped
  MUTEX_UNLOCK( &stream_.mutex );

  if ( result >= 0 ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiAlsa :: abortStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiAlsa::abortStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  stream_.state = STREAM_STOPPED;
  MUTEX_LOCK( &stream_.mutex );

  int result = 0;
  AlsaHandle *apiInfo = (AlsaHandle *) stream_.apiHandle;
  snd_pcm_t **handle = (snd_pcm_t **) apiInfo->handles;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    result = snd_pcm_drop( handle[0] );
    if ( result < 0 ) {
      errorStream_ << "RtApiAlsa::abortStream: error aborting output pcm device, " << snd_strerror( result ) << ".";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

  if ( ( stream_.mode == INPUT || stream_.mode == DUPLEX ) && !apiInfo->synchronized ) {
    result = snd_pcm_drop( handle[1] );
    if ( result < 0 ) {
      errorStream_ << "RtApiAlsa::abortStream: error aborting input pcm device, " << snd_strerror( result ) << ".";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

 unlock:
  apiInfo->runnable = false; // fixes high CPU usage when stopped
  MUTEX_UNLOCK( &stream_.mutex );

  if ( result >= 0 ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiAlsa :: callbackEvent()
{
  AlsaHandle *apiInfo = (AlsaHandle *) stream_.apiHandle;
  if ( stream_.state == STREAM_STOPPED ) {
    MUTEX_LOCK( &stream_.mutex );
    while ( !apiInfo->runnable )
      pthread_cond_wait( &apiInfo->runnable_cv, &stream_.mutex );

    if ( stream_.state != STREAM_RUNNING ) {
      MUTEX_UNLOCK( &stream_.mutex );
      return;
    }
    MUTEX_UNLOCK( &stream_.mutex );
  }

  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiAlsa::callbackEvent(): the stream is closed ... this shouldn't happen!";
    error( RtAudioError::WARNING );
    return;
  }

  int doStopStream = 0;
  RtAudioCallback callback = (RtAudioCallback) stream_.callbackInfo.callback;
  double streamTime = getStreamTime();
  RtAudioStreamStatus status = 0;
  if ( stream_.mode != INPUT && apiInfo->xrun[0] == true ) {
    status |= RTAUDIO_OUTPUT_UNDERFLOW;
    apiInfo->xrun[0] = false;
  }
  if ( stream_.mode != OUTPUT && apiInfo->xrun[1] == true ) {
    status |= RTAUDIO_INPUT_OVERFLOW;
    apiInfo->xrun[1] = false;
  }
  doStopStream = callback( stream_.userBuffer[0], stream_.userBuffer[1],
                           stream_.bufferSize, streamTime, status, stream_.callbackInfo.userData );

  if ( doStopStream == 2 ) {
    abortStream();
    return;
  }

  MUTEX_LOCK( &stream_.mutex );

  // The state might change while waiting on a mutex.
  if ( stream_.state == STREAM_STOPPED ) goto unlock;

  int result;
  char *buffer;
  int channels;
  snd_pcm_t **handle;
  snd_pcm_sframes_t frames;
  RtAudioFormat format;
  handle = (snd_pcm_t **) apiInfo->handles;

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX ) {

    // Setup parameters.
    if ( stream_.doConvertBuffer[1] ) {
      buffer = stream_.deviceBuffer;
      channels = stream_.nDeviceChannels[1];
      format = stream_.deviceFormat[1];
    }
    else {
      buffer = stream_.userBuffer[1];
      channels = stream_.nUserChannels[1];
      format = stream_.userFormat;
    }

    // Read samples from device in interleaved/non-interleaved format.
    if ( stream_.deviceInterleaved[1] )
      result = snd_pcm_readi( handle[1], buffer, stream_.bufferSize );
    else {
      void *bufs[channels];
      size_t offset = stream_.bufferSize * formatBytes( format );
      for ( int i=0; i<channels; i++ )
        bufs[i] = (void *) (buffer + (i * offset));
      result = snd_pcm_readn( handle[1], bufs, stream_.bufferSize );
    }

    if ( result < (int) stream_.bufferSize ) {
      // Either an error or overrun occured.
      if ( result == -EPIPE ) {
        snd_pcm_state_t state = snd_pcm_state( handle[1] );
        if ( state == SND_PCM_STATE_XRUN ) {
          apiInfo->xrun[1] = true;
          result = snd_pcm_prepare( handle[1] );
          if ( result < 0 ) {
            errorStream_ << "RtApiAlsa::callbackEvent: error preparing device after overrun, " << snd_strerror( result ) << ".";
            errorText_ = errorStream_.str();
          }
        }
        else {
          errorStream_ << "RtApiAlsa::callbackEvent: error, current state is " << snd_pcm_state_name( state ) << ", " << snd_strerror( result ) << ".";
          errorText_ = errorStream_.str();
        }
      }
      else {
        errorStream_ << "RtApiAlsa::callbackEvent: audio read error, " << snd_strerror( result ) << ".";
        errorText_ = errorStream_.str();
      }
      error( RtAudioError::WARNING );
      goto tryOutput;
    }

    // Do byte swapping if necessary.
    if ( stream_.doByteSwap[1] )
      byteSwapBuffer( buffer, stream_.bufferSize * channels, format );

    // Do buffer conversion if necessary.
    if ( stream_.doConvertBuffer[1] )
      convertBuffer( stream_.userBuffer[1], stream_.deviceBuffer, stream_.convertInfo[1] );

    // Check stream latency
    result = snd_pcm_delay( handle[1], &frames );
    if ( result == 0 && frames > 0 ) stream_.latency[1] = frames;
  }

 tryOutput:

  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    // Setup parameters and do buffer conversion if necessary.
    if ( stream_.doConvertBuffer[0] ) {
      buffer = stream_.deviceBuffer;
      convertBuffer( buffer, stream_.userBuffer[0], stream_.convertInfo[0] );
      channels = stream_.nDeviceChannels[0];
      format = stream_.deviceFormat[0];
    }
    else {
      buffer = stream_.userBuffer[0];
      channels = stream_.nUserChannels[0];
      format = stream_.userFormat;
    }

    // Do byte swapping if necessary.
    if ( stream_.doByteSwap[0] )
      byteSwapBuffer(buffer, stream_.bufferSize * channels, format);

    // Write samples to device in interleaved/non-interleaved format.
    if ( stream_.deviceInterleaved[0] )
      result = snd_pcm_writei( handle[0], buffer, stream_.bufferSize );
    else {
      void *bufs[channels];
      size_t offset = stream_.bufferSize * formatBytes( format );
      for ( int i=0; i<channels; i++ )
        bufs[i] = (void *) (buffer + (i * offset));
      result = snd_pcm_writen( handle[0], bufs, stream_.bufferSize );
    }

    if ( result < (int) stream_.bufferSize ) {
      // Either an error or underrun occured.
      if ( result == -EPIPE ) {
        snd_pcm_state_t state = snd_pcm_state( handle[0] );
        if ( state == SND_PCM_STATE_XRUN ) {
          apiInfo->xrun[0] = true;
          result = snd_pcm_prepare( handle[0] );
          if ( result < 0 ) {
            errorStream_ << "RtApiAlsa::callbackEvent: error preparing device after underrun, " << snd_strerror( result ) << ".";
            errorText_ = errorStream_.str();
          }
          else
            errorText_ =  "RtApiAlsa::callbackEvent: audio write error, underrun.";
        }
        else {
          errorStream_ << "RtApiAlsa::callbackEvent: error, current state is " << snd_pcm_state_name( state ) << ", " << snd_strerror( result ) << ".";
          errorText_ = errorStream_.str();
        }
      }
      else {
        errorStream_ << "RtApiAlsa::callbackEvent: audio write error, " << snd_strerror( result ) << ".";
        errorText_ = errorStream_.str();
      }
      error( RtAudioError::WARNING );
      goto unlock;
    }

    // Check stream latency
    result = snd_pcm_delay( handle[0], &frames );
    if ( result == 0 && frames > 0 ) stream_.latency[0] = frames;
  }

 unlock:
  MUTEX_UNLOCK( &stream_.mutex );

  RtApi::tickStreamTime();
  if ( doStopStream == 1 ) this->stopStream();
}

static void *alsaCallbackHandler( void *ptr )
{
  CallbackInfo *info = (CallbackInfo *) ptr;
  RtApiAlsa *object = (RtApiAlsa *) info->object;
  bool *isRunning = &info->isRunning;

#ifdef SCHED_RR // Undefined with some OSes (eg: NetBSD 1.6.x with GNU Pthread)
  if ( info->doRealtime ) {
    pthread_t tID = pthread_self();	 // ID of this thread
    sched_param prio = { info->priority }; // scheduling priority of thread
    pthread_setschedparam( tID, SCHED_RR, &prio );
  }
#endif

  while ( *isRunning == true ) {
    pthread_testcancel();
    object->callbackEvent();
  }

  pthread_exit( NULL );
}

//******************** End of __LINUX_ALSA__ *********************//
#endif

#if defined(__LINUX_PULSE__)

// Code written by Peter Meerwald, pmeerw@pmeerw.net
// and Tristan Matthews.

#include <pulse/error.h>
#include <pulse/simple.h>
#include <cstdio>

static const unsigned int SUPPORTED_SAMPLERATES[] = { 8000, 16000, 22050, 32000,
                                                      44100, 48000, 96000, 0};

struct rtaudio_pa_format_mapping_t {
  RtAudioFormat rtaudio_format;
  pa_sample_format_t pa_format;
};

static const rtaudio_pa_format_mapping_t supported_sampleformats[] = {
  {RTAUDIO_SINT16, PA_SAMPLE_S16LE},
  {RTAUDIO_SINT32, PA_SAMPLE_S32LE},
  {RTAUDIO_FLOAT32, PA_SAMPLE_FLOAT32LE},
  {0, PA_SAMPLE_INVALID}};

struct PulseAudioHandle {
  pa_simple *s_play;
  pa_simple *s_rec;
  pthread_t thread;
  pthread_cond_t runnable_cv;
  bool runnable;
  PulseAudioHandle() : s_play(0), s_rec(0), runnable(false) { }
};

RtApiPulse::~RtApiPulse()
{
  if ( stream_.state != STREAM_CLOSED )
    closeStream();
}

unsigned int RtApiPulse::getDeviceCount( void )
{
  return 1;
}

RtAudio::DeviceInfo RtApiPulse::getDeviceInfo( unsigned int /*device*/ )
{
  RtAudio::DeviceInfo info;
  info.probed = true;
  info.name = "PulseAudio";
  info.outputChannels = 2;
  info.inputChannels = 2;
  info.duplexChannels = 2;
  info.isDefaultOutput = true;
  info.isDefaultInput = true;

  for ( const unsigned int *sr = SUPPORTED_SAMPLERATES; *sr; ++sr )
    info.sampleRates.push_back( *sr );

  info.preferredSampleRate = 48000;
  info.nativeFormats = RTAUDIO_SINT16 | RTAUDIO_SINT32 | RTAUDIO_FLOAT32;

  return info;
}

static void *pulseaudio_callback( void * user )
{
  CallbackInfo *cbi = static_cast<CallbackInfo *>( user );
  RtApiPulse *context = static_cast<RtApiPulse *>( cbi->object );
  volatile bool *isRunning = &cbi->isRunning;

  while ( *isRunning ) {
    pthread_testcancel();
    context->callbackEvent();
  }

  pthread_exit( NULL );
}

void RtApiPulse::closeStream( void )
{
  PulseAudioHandle *pah = static_cast<PulseAudioHandle *>( stream_.apiHandle );

  stream_.callbackInfo.isRunning = false;
  if ( pah ) {
    MUTEX_LOCK( &stream_.mutex );
    if ( stream_.state == STREAM_STOPPED ) {
      pah->runnable = true;
      pthread_cond_signal( &pah->runnable_cv );
    }
    MUTEX_UNLOCK( &stream_.mutex );

    pthread_join( pah->thread, 0 );
    if ( pah->s_play ) {
      pa_simple_flush( pah->s_play, NULL );
      pa_simple_free( pah->s_play );
    }
    if ( pah->s_rec )
      pa_simple_free( pah->s_rec );

    pthread_cond_destroy( &pah->runnable_cv );
    delete pah;
    stream_.apiHandle = 0;
  }

  if ( stream_.userBuffer[0] ) {
    free( stream_.userBuffer[0] );
    stream_.userBuffer[0] = 0;
  }
  if ( stream_.userBuffer[1] ) {
    free( stream_.userBuffer[1] );
    stream_.userBuffer[1] = 0;
  }

  stream_.state = STREAM_CLOSED;
  stream_.mode = UNINITIALIZED;
}

void RtApiPulse::callbackEvent( void )
{
  PulseAudioHandle *pah = static_cast<PulseAudioHandle *>( stream_.apiHandle );

  if ( stream_.state == STREAM_STOPPED ) {
    MUTEX_LOCK( &stream_.mutex );
    while ( !pah->runnable )
      pthread_cond_wait( &pah->runnable_cv, &stream_.mutex );

    if ( stream_.state != STREAM_RUNNING ) {
      MUTEX_UNLOCK( &stream_.mutex );
      return;
    }
    MUTEX_UNLOCK( &stream_.mutex );
  }

  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiPulse::callbackEvent(): the stream is closed ... "
      "this shouldn't happen!";
    error( RtAudioError::WARNING );
    return;
  }

  RtAudioCallback callback = (RtAudioCallback) stream_.callbackInfo.callback;
  double streamTime = getStreamTime();
  RtAudioStreamStatus status = 0;
  int doStopStream = callback( stream_.userBuffer[OUTPUT], stream_.userBuffer[INPUT],
                               stream_.bufferSize, streamTime, status,
                               stream_.callbackInfo.userData );

  if ( doStopStream == 2 ) {
    abortStream();
    return;
  }

  MUTEX_LOCK( &stream_.mutex );
  void *pulse_in = stream_.doConvertBuffer[INPUT] ? stream_.deviceBuffer : stream_.userBuffer[INPUT];
  void *pulse_out = stream_.doConvertBuffer[OUTPUT] ? stream_.deviceBuffer : stream_.userBuffer[OUTPUT];

  if ( stream_.state != STREAM_RUNNING )
    goto unlock;

  int pa_error;
  size_t bytes;
  if (stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    if ( stream_.doConvertBuffer[OUTPUT] ) {
        convertBuffer( stream_.deviceBuffer,
                       stream_.userBuffer[OUTPUT],
                       stream_.convertInfo[OUTPUT] );
        bytes = stream_.nDeviceChannels[OUTPUT] * stream_.bufferSize *
                formatBytes( stream_.deviceFormat[OUTPUT] );
    } else
        bytes = stream_.nUserChannels[OUTPUT] * stream_.bufferSize *
                formatBytes( stream_.userFormat );

    if ( pa_simple_write( pah->s_play, pulse_out, bytes, &pa_error ) < 0 ) {
      errorStream_ << "RtApiPulse::callbackEvent: audio write error, " <<
        pa_strerror( pa_error ) << ".";
      errorText_ = errorStream_.str();
      error( RtAudioError::WARNING );
    }
  }

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX) {
    if ( stream_.doConvertBuffer[INPUT] )
      bytes = stream_.nDeviceChannels[INPUT] * stream_.bufferSize *
        formatBytes( stream_.deviceFormat[INPUT] );
    else
      bytes = stream_.nUserChannels[INPUT] * stream_.bufferSize *
        formatBytes( stream_.userFormat );
            
    if ( pa_simple_read( pah->s_rec, pulse_in, bytes, &pa_error ) < 0 ) {
      errorStream_ << "RtApiPulse::callbackEvent: audio read error, " <<
        pa_strerror( pa_error ) << ".";
      errorText_ = errorStream_.str();
      error( RtAudioError::WARNING );
    }
    if ( stream_.doConvertBuffer[INPUT] ) {
      convertBuffer( stream_.userBuffer[INPUT],
                     stream_.deviceBuffer,
                     stream_.convertInfo[INPUT] );
    }
  }

 unlock:
  MUTEX_UNLOCK( &stream_.mutex );
  RtApi::tickStreamTime();

  if ( doStopStream == 1 )
    stopStream();
}

void RtApiPulse::startStream( void )
{
  PulseAudioHandle *pah = static_cast<PulseAudioHandle *>( stream_.apiHandle );

  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiPulse::startStream(): the stream is not open!";
    error( RtAudioError::INVALID_USE );
    return;
  }
  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiPulse::startStream(): the stream is already running!";
    error( RtAudioError::WARNING );
    return;
  }

  MUTEX_LOCK( &stream_.mutex );

  stream_.state = STREAM_RUNNING;

  pah->runnable = true;
  pthread_cond_signal( &pah->runnable_cv );
  MUTEX_UNLOCK( &stream_.mutex );
}

void RtApiPulse::stopStream( void )
{
  PulseAudioHandle *pah = static_cast<PulseAudioHandle *>( stream_.apiHandle );

  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiPulse::stopStream(): the stream is not open!";
    error( RtAudioError::INVALID_USE );
    return;
  }
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiPulse::stopStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  stream_.state = STREAM_STOPPED;
  MUTEX_LOCK( &stream_.mutex );

  if ( pah && pah->s_play ) {
    int pa_error;
    if ( pa_simple_drain( pah->s_play, &pa_error ) < 0 ) {
      errorStream_ << "RtApiPulse::stopStream: error draining output device, " <<
        pa_strerror( pa_error ) << ".";
      errorText_ = errorStream_.str();
      MUTEX_UNLOCK( &stream_.mutex );
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }
  }

  stream_.state = STREAM_STOPPED;
  MUTEX_UNLOCK( &stream_.mutex );
}

void RtApiPulse::abortStream( void )
{
  PulseAudioHandle *pah = static_cast<PulseAudioHandle*>( stream_.apiHandle );

  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiPulse::abortStream(): the stream is not open!";
    error( RtAudioError::INVALID_USE );
    return;
  }
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiPulse::abortStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  stream_.state = STREAM_STOPPED;
  MUTEX_LOCK( &stream_.mutex );

  if ( pah && pah->s_play ) {
    int pa_error;
    if ( pa_simple_flush( pah->s_play, &pa_error ) < 0 ) {
      errorStream_ << "RtApiPulse::abortStream: error flushing output device, " <<
        pa_strerror( pa_error ) << ".";
      errorText_ = errorStream_.str();
      MUTEX_UNLOCK( &stream_.mutex );
      error( RtAudioError::SYSTEM_ERROR );
      return;
    }
  }

  stream_.state = STREAM_STOPPED;
  MUTEX_UNLOCK( &stream_.mutex );
}

bool RtApiPulse::probeDeviceOpen( unsigned int device, StreamMode mode,
                                  unsigned int channels, unsigned int firstChannel,
                                  unsigned int sampleRate, RtAudioFormat format,
                                  unsigned int *bufferSize, RtAudio::StreamOptions *options )
{
  PulseAudioHandle *pah = 0;
  unsigned long bufferBytes = 0;
  pa_sample_spec ss;

  if ( device != 0 ) return false;
  if ( mode != INPUT && mode != OUTPUT ) return false;
  if ( channels != 1 && channels != 2 ) {
    errorText_ = "RtApiPulse::probeDeviceOpen: unsupported number of channels.";
    return false;
  }
  ss.channels = channels;

  if ( firstChannel != 0 ) return false;

  bool sr_found = false;
  for ( const unsigned int *sr = SUPPORTED_SAMPLERATES; *sr; ++sr ) {
    if ( sampleRate == *sr ) {
      sr_found = true;
      stream_.sampleRate = sampleRate;
      ss.rate = sampleRate;
      break;
    }
  }
  if ( !sr_found ) {
    errorText_ = "RtApiPulse::probeDeviceOpen: unsupported sample rate.";
    return false;
  }

  bool sf_found = 0;
  for ( const rtaudio_pa_format_mapping_t *sf = supported_sampleformats;
        sf->rtaudio_format && sf->pa_format != PA_SAMPLE_INVALID; ++sf ) {
    if ( format == sf->rtaudio_format ) {
      sf_found = true;
      stream_.userFormat = sf->rtaudio_format;
      stream_.deviceFormat[mode] = stream_.userFormat;
      ss.format = sf->pa_format;
      break;
    }
  }
  if ( !sf_found ) { // Use internal data format conversion.
    stream_.userFormat = format;
    stream_.deviceFormat[mode] = RTAUDIO_FLOAT32;
    ss.format = PA_SAMPLE_FLOAT32LE;
  }

  // Set other stream parameters.
  if ( options && options->flags & RTAUDIO_NONINTERLEAVED ) stream_.userInterleaved = false;
  else stream_.userInterleaved = true;
  stream_.deviceInterleaved[mode] = true;
  stream_.nBuffers = 1;
  stream_.doByteSwap[mode] = false;
  stream_.nUserChannels[mode] = channels;
  stream_.nDeviceChannels[mode] = channels + firstChannel;
  stream_.channelOffset[mode] = 0;
  std::string streamName = "RtAudio";

  // Set flags for buffer conversion.
  stream_.doConvertBuffer[mode] = false;
  if ( stream_.userFormat != stream_.deviceFormat[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.nUserChannels[mode] < stream_.nDeviceChannels[mode] )
    stream_.doConvertBuffer[mode] = true;

  // Allocate necessary internal buffers.
  bufferBytes = stream_.nUserChannels[mode] * *bufferSize * formatBytes( stream_.userFormat );
  stream_.userBuffer[mode] = (char *) calloc( bufferBytes, 1 );
  if ( stream_.userBuffer[mode] == NULL ) {
    errorText_ = "RtApiPulse::probeDeviceOpen: error allocating user buffer memory.";
    goto error;
  }
  stream_.bufferSize = *bufferSize;

  if ( stream_.doConvertBuffer[mode] ) {

    bool makeBuffer = true;
    bufferBytes = stream_.nDeviceChannels[mode] * formatBytes( stream_.deviceFormat[mode] );
    if ( mode == INPUT ) {
      if ( stream_.mode == OUTPUT && stream_.deviceBuffer ) {
        unsigned long bytesOut = stream_.nDeviceChannels[0] * formatBytes( stream_.deviceFormat[0] );
        if ( bufferBytes <= bytesOut ) makeBuffer = false;
      }
    }

    if ( makeBuffer ) {
      bufferBytes *= *bufferSize;
      if ( stream_.deviceBuffer ) free( stream_.deviceBuffer );
      stream_.deviceBuffer = (char *) calloc( bufferBytes, 1 );
      if ( stream_.deviceBuffer == NULL ) {
        errorText_ = "RtApiPulse::probeDeviceOpen: error allocating device buffer memory.";
        goto error;
      }
    }
  }

  stream_.device[mode] = device;

  // Setup the buffer conversion information structure.
  if ( stream_.doConvertBuffer[mode] ) setConvertInfo( mode, firstChannel );

  if ( !stream_.apiHandle ) {
    PulseAudioHandle *pah = new PulseAudioHandle;
    if ( !pah ) {
      errorText_ = "RtApiPulse::probeDeviceOpen: error allocating memory for handle.";
      goto error;
    }

    stream_.apiHandle = pah;
    if ( pthread_cond_init( &pah->runnable_cv, NULL ) != 0 ) {
      errorText_ = "RtApiPulse::probeDeviceOpen: error creating condition variable.";
      goto error;
    }
  }
  pah = static_cast<PulseAudioHandle *>( stream_.apiHandle );

  int error;
  if ( options && !options->streamName.empty() ) streamName = options->streamName;
  switch ( mode ) {
  case INPUT:
    pa_buffer_attr buffer_attr;
    buffer_attr.fragsize = bufferBytes;
    buffer_attr.maxlength = -1;

    pah->s_rec = pa_simple_new( NULL, streamName.c_str(), PA_STREAM_RECORD, NULL, "Record", &ss, NULL, &buffer_attr, &error );
    if ( !pah->s_rec ) {
      errorText_ = "RtApiPulse::probeDeviceOpen: error connecting input to PulseAudio server.";
      goto error;
    }
    break;
  case OUTPUT:
    pah->s_play = pa_simple_new( NULL, streamName.c_str(), PA_STREAM_PLAYBACK, NULL, "Playback", &ss, NULL, NULL, &error );
    if ( !pah->s_play ) {
      errorText_ = "RtApiPulse::probeDeviceOpen: error connecting output to PulseAudio server.";
      goto error;
    }
    break;
  default:
    goto error;
  }

  if ( stream_.mode == UNINITIALIZED )
    stream_.mode = mode;
  else if ( stream_.mode == mode )
    goto error;
  else
    stream_.mode = DUPLEX;

  if ( !stream_.callbackInfo.isRunning ) {
    stream_.callbackInfo.object = this;
    stream_.callbackInfo.isRunning = true;
    if ( pthread_create( &pah->thread, NULL, pulseaudio_callback, (void *)&stream_.callbackInfo) != 0 ) {
      errorText_ = "RtApiPulse::probeDeviceOpen: error creating thread.";
      goto error;
    }
  }

  stream_.state = STREAM_STOPPED;
  return true;
 
 error:
  if ( pah && stream_.callbackInfo.isRunning ) {
    pthread_cond_destroy( &pah->runnable_cv );
    delete pah;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  return FAILURE;
}

//******************** End of __LINUX_PULSE__ *********************//
#endif

#if defined(__LINUX_OSS__)

#include <unistd.h>
#include <sys/ioctl.h>
#include <unistd.h>
#include <fcntl.h>
#include <sys/soundcard.h>
#include <errno.h>
#include <math.h>

static void *ossCallbackHandler(void * ptr);

// A structure to hold various information related to the OSS API
// implementation.
struct OssHandle {
  int id[2];    // device ids
  bool xrun[2];
  bool triggered;
  pthread_cond_t runnable;

  OssHandle()
    :triggered(false) { id[0] = 0; id[1] = 0; xrun[0] = false; xrun[1] = false; }
};

RtApiOss :: RtApiOss()
{
  // Nothing to do here.
}

RtApiOss :: ~RtApiOss()
{
  if ( stream_.state != STREAM_CLOSED ) closeStream();
}

unsigned int RtApiOss :: getDeviceCount( void )
{
  int mixerfd = open( "/dev/mixer", O_RDWR, 0 );
  if ( mixerfd == -1 ) {
    errorText_ = "RtApiOss::getDeviceCount: error opening '/dev/mixer'.";
    error( RtAudioError::WARNING );
    return 0;
  }

  oss_sysinfo sysinfo;
  if ( ioctl( mixerfd, SNDCTL_SYSINFO, &sysinfo ) == -1 ) {
    close( mixerfd );
    errorText_ = "RtApiOss::getDeviceCount: error getting sysinfo, OSS version >= 4.0 is required.";
    error( RtAudioError::WARNING );
    return 0;
  }

  close( mixerfd );
  return sysinfo.numaudios;
}

RtAudio::DeviceInfo RtApiOss :: getDeviceInfo( unsigned int device )
{
  RtAudio::DeviceInfo info;
  info.probed = false;

  int mixerfd = open( "/dev/mixer", O_RDWR, 0 );
  if ( mixerfd == -1 ) {
    errorText_ = "RtApiOss::getDeviceInfo: error opening '/dev/mixer'.";
    error( RtAudioError::WARNING );
    return info;
  }

  oss_sysinfo sysinfo;
  int result = ioctl( mixerfd, SNDCTL_SYSINFO, &sysinfo );
  if ( result == -1 ) {
    close( mixerfd );
    errorText_ = "RtApiOss::getDeviceInfo: error getting sysinfo, OSS version >= 4.0 is required.";
    error( RtAudioError::WARNING );
    return info;
  }

  unsigned nDevices = sysinfo.numaudios;
  if ( nDevices == 0 ) {
    close( mixerfd );
    errorText_ = "RtApiOss::getDeviceInfo: no devices found!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  if ( device >= nDevices ) {
    close( mixerfd );
    errorText_ = "RtApiOss::getDeviceInfo: device ID is invalid!";
    error( RtAudioError::INVALID_USE );
    return info;
  }

  oss_audioinfo ainfo;
  ainfo.dev = device;
  result = ioctl( mixerfd, SNDCTL_AUDIOINFO, &ainfo );
  close( mixerfd );
  if ( result == -1 ) {
    errorStream_ << "RtApiOss::getDeviceInfo: error getting device (" << ainfo.name << ") info.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Probe channels
  if ( ainfo.caps & PCM_CAP_OUTPUT ) info.outputChannels = ainfo.max_channels;
  if ( ainfo.caps & PCM_CAP_INPUT ) info.inputChannels = ainfo.max_channels;
  if ( ainfo.caps & PCM_CAP_DUPLEX ) {
    if ( info.outputChannels > 0 && info.inputChannels > 0 && ainfo.caps & PCM_CAP_DUPLEX )
      info.duplexChannels = (info.outputChannels > info.inputChannels) ? info.inputChannels : info.outputChannels;
  }

  // Probe data formats ... do for input
  unsigned long mask = ainfo.iformats;
  if ( mask & AFMT_S16_LE || mask & AFMT_S16_BE )
    info.nativeFormats |= RTAUDIO_SINT16;
  if ( mask & AFMT_S8 )
    info.nativeFormats |= RTAUDIO_SINT8;
  if ( mask & AFMT_S32_LE || mask & AFMT_S32_BE )
    info.nativeFormats |= RTAUDIO_SINT32;
#ifdef AFMT_FLOAT
  if ( mask & AFMT_FLOAT )
    info.nativeFormats |= RTAUDIO_FLOAT32;
#endif
  if ( mask & AFMT_S24_LE || mask & AFMT_S24_BE )
    info.nativeFormats |= RTAUDIO_SINT24;

  // Check that we have at least one supported format
  if ( info.nativeFormats == 0 ) {
    errorStream_ << "RtApiOss::getDeviceInfo: device (" << ainfo.name << ") data format not supported by RtAudio.";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
    return info;
  }

  // Probe the supported sample rates.
  info.sampleRates.clear();
  if ( ainfo.nrates ) {
    for ( unsigned int i=0; i<ainfo.nrates; i++ ) {
      for ( unsigned int k=0; k<MAX_SAMPLE_RATES; k++ ) {
        if ( ainfo.rates[i] == SAMPLE_RATES[k] ) {
          info.sampleRates.push_back( SAMPLE_RATES[k] );

          if ( !info.preferredSampleRate || ( SAMPLE_RATES[k] <= 48000 && SAMPLE_RATES[k] > info.preferredSampleRate ) )
            info.preferredSampleRate = SAMPLE_RATES[k];

          break;
        }
      }
    }
  }
  else {
    // Check min and max rate values;
    for ( unsigned int k=0; k<MAX_SAMPLE_RATES; k++ ) {
      if ( ainfo.min_rate <= (int) SAMPLE_RATES[k] && ainfo.max_rate >= (int) SAMPLE_RATES[k] ) {
        info.sampleRates.push_back( SAMPLE_RATES[k] );

        if ( !info.preferredSampleRate || ( SAMPLE_RATES[k] <= 48000 && SAMPLE_RATES[k] > info.preferredSampleRate ) )
          info.preferredSampleRate = SAMPLE_RATES[k];
      }
    }
  }

  if ( info.sampleRates.size() == 0 ) {
    errorStream_ << "RtApiOss::getDeviceInfo: no supported sample rates found for device (" << ainfo.name << ").";
    errorText_ = errorStream_.str();
    error( RtAudioError::WARNING );
  }
  else {
    info.probed = true;
    info.name = ainfo.name;
  }

  return info;
}


bool RtApiOss :: probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                                  unsigned int firstChannel, unsigned int sampleRate,
                                  RtAudioFormat format, unsigned int *bufferSize,
                                  RtAudio::StreamOptions *options )
{
  int mixerfd = open( "/dev/mixer", O_RDWR, 0 );
  if ( mixerfd == -1 ) {
    errorText_ = "RtApiOss::probeDeviceOpen: error opening '/dev/mixer'.";
    return FAILURE;
  }

  oss_sysinfo sysinfo;
  int result = ioctl( mixerfd, SNDCTL_SYSINFO, &sysinfo );
  if ( result == -1 ) {
    close( mixerfd );
    errorText_ = "RtApiOss::probeDeviceOpen: error getting sysinfo, OSS version >= 4.0 is required.";
    return FAILURE;
  }

  unsigned nDevices = sysinfo.numaudios;
  if ( nDevices == 0 ) {
    // This should not happen because a check is made before this function is called.
    close( mixerfd );
    errorText_ = "RtApiOss::probeDeviceOpen: no devices found!";
    return FAILURE;
  }

  if ( device >= nDevices ) {
    // This should not happen because a check is made before this function is called.
    close( mixerfd );
    errorText_ = "RtApiOss::probeDeviceOpen: device ID is invalid!";
    return FAILURE;
  }

  oss_audioinfo ainfo;
  ainfo.dev = device;
  result = ioctl( mixerfd, SNDCTL_AUDIOINFO, &ainfo );
  close( mixerfd );
  if ( result == -1 ) {
    errorStream_ << "RtApiOss::getDeviceInfo: error getting device (" << ainfo.name << ") info.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Check if device supports input or output
  if ( ( mode == OUTPUT && !( ainfo.caps & PCM_CAP_OUTPUT ) ) ||
       ( mode == INPUT && !( ainfo.caps & PCM_CAP_INPUT ) ) ) {
    if ( mode == OUTPUT )
      errorStream_ << "RtApiOss::probeDeviceOpen: device (" << ainfo.name << ") does not support output.";
    else
      errorStream_ << "RtApiOss::probeDeviceOpen: device (" << ainfo.name << ") does not support input.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  int flags = 0;
  OssHandle *handle = (OssHandle *) stream_.apiHandle;
  if ( mode == OUTPUT )
    flags |= O_WRONLY;
  else { // mode == INPUT
    if (stream_.mode == OUTPUT && stream_.device[0] == device) {
      // We just set the same device for playback ... close and reopen for duplex (OSS only).
      close( handle->id[0] );
      handle->id[0] = 0;
      if ( !( ainfo.caps & PCM_CAP_DUPLEX ) ) {
        errorStream_ << "RtApiOss::probeDeviceOpen: device (" << ainfo.name << ") does not support duplex mode.";
        errorText_ = errorStream_.str();
        return FAILURE;
      }
      // Check that the number previously set channels is the same.
      if ( stream_.nUserChannels[0] != channels ) {
        errorStream_ << "RtApiOss::probeDeviceOpen: input/output channels must be equal for OSS duplex device (" << ainfo.name << ").";
        errorText_ = errorStream_.str();
        return FAILURE;
      }
      flags |= O_RDWR;
    }
    else
      flags |= O_RDONLY;
  }

  // Set exclusive access if specified.
  if ( options && options->flags & RTAUDIO_HOG_DEVICE ) flags |= O_EXCL;

  // Try to open the device.
  int fd;
  fd = open( ainfo.devnode, flags, 0 );
  if ( fd == -1 ) {
    if ( errno == EBUSY )
      errorStream_ << "RtApiOss::probeDeviceOpen: device (" << ainfo.name << ") is busy.";
    else
      errorStream_ << "RtApiOss::probeDeviceOpen: error opening device (" << ainfo.name << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // For duplex operation, specifically set this mode (this doesn't seem to work).
  /*
    if ( flags | O_RDWR ) {
    result = ioctl( fd, SNDCTL_DSP_SETDUPLEX, NULL );
    if ( result == -1) {
    errorStream_ << "RtApiOss::probeDeviceOpen: error setting duplex mode for device (" << ainfo.name << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
    }
    }
  */

  // Check the device channel support.
  stream_.nUserChannels[mode] = channels;
  if ( ainfo.max_channels < (int)(channels + firstChannel) ) {
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: the device (" << ainfo.name << ") does not support requested channel parameters.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Set the number of channels.
  int deviceChannels = channels + firstChannel;
  result = ioctl( fd, SNDCTL_DSP_CHANNELS, &deviceChannels );
  if ( result == -1 || deviceChannels < (int)(channels + firstChannel) ) {
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: error setting channel parameters on device (" << ainfo.name << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }
  stream_.nDeviceChannels[mode] = deviceChannels;

  // Get the data format mask
  int mask;
  result = ioctl( fd, SNDCTL_DSP_GETFMTS, &mask );
  if ( result == -1 ) {
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: error getting device (" << ainfo.name << ") data formats.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Determine how to set the device format.
  stream_.userFormat = format;
  int deviceFormat = -1;
  stream_.doByteSwap[mode] = false;
  if ( format == RTAUDIO_SINT8 ) {
    if ( mask & AFMT_S8 ) {
      deviceFormat = AFMT_S8;
      stream_.deviceFormat[mode] = RTAUDIO_SINT8;
    }
  }
  else if ( format == RTAUDIO_SINT16 ) {
    if ( mask & AFMT_S16_NE ) {
      deviceFormat = AFMT_S16_NE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT16;
    }
    else if ( mask & AFMT_S16_OE ) {
      deviceFormat = AFMT_S16_OE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT16;
      stream_.doByteSwap[mode] = true;
    }
  }
  else if ( format == RTAUDIO_SINT24 ) {
    if ( mask & AFMT_S24_NE ) {
      deviceFormat = AFMT_S24_NE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT24;
    }
    else if ( mask & AFMT_S24_OE ) {
      deviceFormat = AFMT_S24_OE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT24;
      stream_.doByteSwap[mode] = true;
    }
  }
  else if ( format == RTAUDIO_SINT32 ) {
    if ( mask & AFMT_S32_NE ) {
      deviceFormat = AFMT_S32_NE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT32;
    }
    else if ( mask & AFMT_S32_OE ) {
      deviceFormat = AFMT_S32_OE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT32;
      stream_.doByteSwap[mode] = true;
    }
  }

  if ( deviceFormat == -1 ) {
    // The user requested format is not natively supported by the device.
    if ( mask & AFMT_S16_NE ) {
      deviceFormat = AFMT_S16_NE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT16;
    }
    else if ( mask & AFMT_S32_NE ) {
      deviceFormat = AFMT_S32_NE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT32;
    }
    else if ( mask & AFMT_S24_NE ) {
      deviceFormat = AFMT_S24_NE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT24;
    }
    else if ( mask & AFMT_S16_OE ) {
      deviceFormat = AFMT_S16_OE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT16;
      stream_.doByteSwap[mode] = true;
    }
    else if ( mask & AFMT_S32_OE ) {
      deviceFormat = AFMT_S32_OE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT32;
      stream_.doByteSwap[mode] = true;
    }
    else if ( mask & AFMT_S24_OE ) {
      deviceFormat = AFMT_S24_OE;
      stream_.deviceFormat[mode] = RTAUDIO_SINT24;
      stream_.doByteSwap[mode] = true;
    }
    else if ( mask & AFMT_S8) {
      deviceFormat = AFMT_S8;
      stream_.deviceFormat[mode] = RTAUDIO_SINT8;
    }
  }

  if ( stream_.deviceFormat[mode] == 0 ) {
    // This really shouldn't happen ...
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: device (" << ainfo.name << ") data format not supported by RtAudio.";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Set the data format.
  int temp = deviceFormat;
  result = ioctl( fd, SNDCTL_DSP_SETFMT, &deviceFormat );
  if ( result == -1 || deviceFormat != temp ) {
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: error setting data format on device (" << ainfo.name << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Attempt to set the buffer size.  According to OSS, the minimum
  // number of buffers is two.  The supposed minimum buffer size is 16
  // bytes, so that will be our lower bound.  The argument to this
  // call is in the form 0xMMMMSSSS (hex), where the buffer size (in
  // bytes) is given as 2^SSSS and the number of buffers as 2^MMMM.
  // We'll check the actual value used near the end of the setup
  // procedure.
  int ossBufferBytes = *bufferSize * formatBytes( stream_.deviceFormat[mode] ) * deviceChannels;
  if ( ossBufferBytes < 16 ) ossBufferBytes = 16;
  int buffers = 0;
  if ( options ) buffers = options->numberOfBuffers;
  if ( options && options->flags & RTAUDIO_MINIMIZE_LATENCY ) buffers = 2;
  if ( buffers < 2 ) buffers = 3;
  temp = ((int) buffers << 16) + (int)( log10( (double)ossBufferBytes ) / log10( 2.0 ) );
  result = ioctl( fd, SNDCTL_DSP_SETFRAGMENT, &temp );
  if ( result == -1 ) {
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: error setting buffer size on device (" << ainfo.name << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }
  stream_.nBuffers = buffers;

  // Save buffer size (in sample frames).
  *bufferSize = ossBufferBytes / ( formatBytes(stream_.deviceFormat[mode]) * deviceChannels );
  stream_.bufferSize = *bufferSize;

  // Set the sample rate.
  int srate = sampleRate;
  result = ioctl( fd, SNDCTL_DSP_SPEED, &srate );
  if ( result == -1 ) {
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: error setting sample rate (" << sampleRate << ") on device (" << ainfo.name << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }

  // Verify the sample rate setup worked.
  if ( abs( srate - (int)sampleRate ) > 100 ) {
    close( fd );
    errorStream_ << "RtApiOss::probeDeviceOpen: device (" << ainfo.name << ") does not support sample rate (" << sampleRate << ").";
    errorText_ = errorStream_.str();
    return FAILURE;
  }
  stream_.sampleRate = sampleRate;

  if ( mode == INPUT && stream_.mode == OUTPUT && stream_.device[0] == device) {
    // We're doing duplex setup here.
    stream_.deviceFormat[0] = stream_.deviceFormat[1];
    stream_.nDeviceChannels[0] = deviceChannels;
  }

  // Set interleaving parameters.
  stream_.userInterleaved = true;
  stream_.deviceInterleaved[mode] =  true;
  if ( options && options->flags & RTAUDIO_NONINTERLEAVED )
    stream_.userInterleaved = false;

  // Set flags for buffer conversion
  stream_.doConvertBuffer[mode] = false;
  if ( stream_.userFormat != stream_.deviceFormat[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.nUserChannels[mode] < stream_.nDeviceChannels[mode] )
    stream_.doConvertBuffer[mode] = true;
  if ( stream_.userInterleaved != stream_.deviceInterleaved[mode] &&
       stream_.nUserChannels[mode] > 1 )
    stream_.doConvertBuffer[mode] = true;

  // Allocate the stream handles if necessary and then save.
  if ( stream_.apiHandle == 0 ) {
    try {
      handle = new OssHandle;
    }
    catch ( std::bad_alloc& ) {
      errorText_ = "RtApiOss::probeDeviceOpen: error allocating OssHandle memory.";
      goto error;
    }

    if ( pthread_cond_init( &handle->runnable, NULL ) ) {
      errorText_ = "RtApiOss::probeDeviceOpen: error initializing pthread condition variable.";
      goto error;
    }

    stream_.apiHandle = (void *) handle;
  }
  else {
    handle = (OssHandle *) stream_.apiHandle;
  }
  handle->id[mode] = fd;

  // Allocate necessary internal buffers.
  unsigned long bufferBytes;
  bufferBytes = stream_.nUserChannels[mode] * *bufferSize * formatBytes( stream_.userFormat );
  stream_.userBuffer[mode] = (char *) calloc( bufferBytes, 1 );
  if ( stream_.userBuffer[mode] == NULL ) {
    errorText_ = "RtApiOss::probeDeviceOpen: error allocating user buffer memory.";
    goto error;
  }

  if ( stream_.doConvertBuffer[mode] ) {

    bool makeBuffer = true;
    bufferBytes = stream_.nDeviceChannels[mode] * formatBytes( stream_.deviceFormat[mode] );
    if ( mode == INPUT ) {
      if ( stream_.mode == OUTPUT && stream_.deviceBuffer ) {
        unsigned long bytesOut = stream_.nDeviceChannels[0] * formatBytes( stream_.deviceFormat[0] );
        if ( bufferBytes <= bytesOut ) makeBuffer = false;
      }
    }

    if ( makeBuffer ) {
      bufferBytes *= *bufferSize;
      if ( stream_.deviceBuffer ) free( stream_.deviceBuffer );
      stream_.deviceBuffer = (char *) calloc( bufferBytes, 1 );
      if ( stream_.deviceBuffer == NULL ) {
        errorText_ = "RtApiOss::probeDeviceOpen: error allocating device buffer memory.";
        goto error;
      }
    }
  }

  stream_.device[mode] = device;
  stream_.state = STREAM_STOPPED;

  // Setup the buffer conversion information structure.
  if ( stream_.doConvertBuffer[mode] ) setConvertInfo( mode, firstChannel );

  // Setup thread if necessary.
  if ( stream_.mode == OUTPUT && mode == INPUT ) {
    // We had already set up an output stream.
    stream_.mode = DUPLEX;
    if ( stream_.device[0] == device ) handle->id[0] = fd;
  }
  else {
    stream_.mode = mode;

    // Setup callback thread.
    stream_.callbackInfo.object = (void *) this;

    // Set the thread attributes for joinable and realtime scheduling
    // priority.  The higher priority will only take affect if the
    // program is run as root or suid.
    pthread_attr_t attr;
    pthread_attr_init( &attr );
    pthread_attr_setdetachstate( &attr, PTHREAD_CREATE_JOINABLE );
#ifdef SCHED_RR // Undefined with some OSes (eg: NetBSD 1.6.x with GNU Pthread)
    if ( options && options->flags & RTAUDIO_SCHEDULE_REALTIME ) {
      struct sched_param param;
      int priority = options->priority;
      int min = sched_get_priority_min( SCHED_RR );
      int max = sched_get_priority_max( SCHED_RR );
      if ( priority < min ) priority = min;
      else if ( priority > max ) priority = max;
      param.sched_priority = priority;
      pthread_attr_setschedparam( &attr, &param );
      pthread_attr_setschedpolicy( &attr, SCHED_RR );
    }
    else
      pthread_attr_setschedpolicy( &attr, SCHED_OTHER );
#else
    pthread_attr_setschedpolicy( &attr, SCHED_OTHER );
#endif

    stream_.callbackInfo.isRunning = true;
    result = pthread_create( &stream_.callbackInfo.thread, &attr, ossCallbackHandler, &stream_.callbackInfo );
    pthread_attr_destroy( &attr );
    if ( result ) {
      stream_.callbackInfo.isRunning = false;
      errorText_ = "RtApiOss::error creating callback thread!";
      goto error;
    }
  }

  return SUCCESS;

 error:
  if ( handle ) {
    pthread_cond_destroy( &handle->runnable );
    if ( handle->id[0] ) close( handle->id[0] );
    if ( handle->id[1] ) close( handle->id[1] );
    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  return FAILURE;
}

void RtApiOss :: closeStream()
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiOss::closeStream(): no open stream to close!";
    error( RtAudioError::WARNING );
    return;
  }

  OssHandle *handle = (OssHandle *) stream_.apiHandle;
  stream_.callbackInfo.isRunning = false;
  MUTEX_LOCK( &stream_.mutex );
  if ( stream_.state == STREAM_STOPPED )
    pthread_cond_signal( &handle->runnable );
  MUTEX_UNLOCK( &stream_.mutex );
  pthread_join( stream_.callbackInfo.thread, NULL );

  if ( stream_.state == STREAM_RUNNING ) {
    if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX )
      ioctl( handle->id[0], SNDCTL_DSP_HALT, 0 );
    else
      ioctl( handle->id[1], SNDCTL_DSP_HALT, 0 );
    stream_.state = STREAM_STOPPED;
  }

  if ( handle ) {
    pthread_cond_destroy( &handle->runnable );
    if ( handle->id[0] ) close( handle->id[0] );
    if ( handle->id[1] ) close( handle->id[1] );
    delete handle;
    stream_.apiHandle = 0;
  }

  for ( int i=0; i<2; i++ ) {
    if ( stream_.userBuffer[i] ) {
      free( stream_.userBuffer[i] );
      stream_.userBuffer[i] = 0;
    }
  }

  if ( stream_.deviceBuffer ) {
    free( stream_.deviceBuffer );
    stream_.deviceBuffer = 0;
  }

  stream_.mode = UNINITIALIZED;
  stream_.state = STREAM_CLOSED;
}

void RtApiOss :: startStream()
{
  verifyStream();
  if ( stream_.state == STREAM_RUNNING ) {
    errorText_ = "RtApiOss::startStream(): the stream is already running!";
    error( RtAudioError::WARNING );
    return;
  }

  MUTEX_LOCK( &stream_.mutex );

  stream_.state = STREAM_RUNNING;

  // No need to do anything else here ... OSS automatically starts
  // when fed samples.

  MUTEX_UNLOCK( &stream_.mutex );

  OssHandle *handle = (OssHandle *) stream_.apiHandle;
  pthread_cond_signal( &handle->runnable );
}

void RtApiOss :: stopStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiOss::stopStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  MUTEX_LOCK( &stream_.mutex );

  // The state might change while waiting on a mutex.
  if ( stream_.state == STREAM_STOPPED ) {
    MUTEX_UNLOCK( &stream_.mutex );
    return;
  }

  int result = 0;
  OssHandle *handle = (OssHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    // Flush the output with zeros a few times.
    char *buffer;
    int samples;
    RtAudioFormat format;

    if ( stream_.doConvertBuffer[0] ) {
      buffer = stream_.deviceBuffer;
      samples = stream_.bufferSize * stream_.nDeviceChannels[0];
      format = stream_.deviceFormat[0];
    }
    else {
      buffer = stream_.userBuffer[0];
      samples = stream_.bufferSize * stream_.nUserChannels[0];
      format = stream_.userFormat;
    }

    memset( buffer, 0, samples * formatBytes(format) );
    for ( unsigned int i=0; i<stream_.nBuffers+1; i++ ) {
      result = write( handle->id[0], buffer, samples * formatBytes(format) );
      if ( result == -1 ) {
        errorText_ = "RtApiOss::stopStream: audio write error.";
        error( RtAudioError::WARNING );
      }
    }

    result = ioctl( handle->id[0], SNDCTL_DSP_HALT, 0 );
    if ( result == -1 ) {
      errorStream_ << "RtApiOss::stopStream: system error stopping callback procedure on device (" << stream_.device[0] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
    handle->triggered = false;
  }

  if ( stream_.mode == INPUT || ( stream_.mode == DUPLEX && handle->id[0] != handle->id[1] ) ) {
    result = ioctl( handle->id[1], SNDCTL_DSP_HALT, 0 );
    if ( result == -1 ) {
      errorStream_ << "RtApiOss::stopStream: system error stopping input callback procedure on device (" << stream_.device[0] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

 unlock:
  stream_.state = STREAM_STOPPED;
  MUTEX_UNLOCK( &stream_.mutex );

  if ( result != -1 ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiOss :: abortStream()
{
  verifyStream();
  if ( stream_.state == STREAM_STOPPED ) {
    errorText_ = "RtApiOss::abortStream(): the stream is already stopped!";
    error( RtAudioError::WARNING );
    return;
  }

  MUTEX_LOCK( &stream_.mutex );

  // The state might change while waiting on a mutex.
  if ( stream_.state == STREAM_STOPPED ) {
    MUTEX_UNLOCK( &stream_.mutex );
    return;
  }

  int result = 0;
  OssHandle *handle = (OssHandle *) stream_.apiHandle;
  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {
    result = ioctl( handle->id[0], SNDCTL_DSP_HALT, 0 );
    if ( result == -1 ) {
      errorStream_ << "RtApiOss::abortStream: system error stopping callback procedure on device (" << stream_.device[0] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
    handle->triggered = false;
  }

  if ( stream_.mode == INPUT || ( stream_.mode == DUPLEX && handle->id[0] != handle->id[1] ) ) {
    result = ioctl( handle->id[1], SNDCTL_DSP_HALT, 0 );
    if ( result == -1 ) {
      errorStream_ << "RtApiOss::abortStream: system error stopping input callback procedure on device (" << stream_.device[0] << ").";
      errorText_ = errorStream_.str();
      goto unlock;
    }
  }

 unlock:
  stream_.state = STREAM_STOPPED;
  MUTEX_UNLOCK( &stream_.mutex );

  if ( result != -1 ) return;
  error( RtAudioError::SYSTEM_ERROR );
}

void RtApiOss :: callbackEvent()
{
  OssHandle *handle = (OssHandle *) stream_.apiHandle;
  if ( stream_.state == STREAM_STOPPED ) {
    MUTEX_LOCK( &stream_.mutex );
    pthread_cond_wait( &handle->runnable, &stream_.mutex );
    if ( stream_.state != STREAM_RUNNING ) {
      MUTEX_UNLOCK( &stream_.mutex );
      return;
    }
    MUTEX_UNLOCK( &stream_.mutex );
  }

  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApiOss::callbackEvent(): the stream is closed ... this shouldn't happen!";
    error( RtAudioError::WARNING );
    return;
  }

  // Invoke user callback to get fresh output data.
  int doStopStream = 0;
  RtAudioCallback callback = (RtAudioCallback) stream_.callbackInfo.callback;
  double streamTime = getStreamTime();
  RtAudioStreamStatus status = 0;
  if ( stream_.mode != INPUT && handle->xrun[0] == true ) {
    status |= RTAUDIO_OUTPUT_UNDERFLOW;
    handle->xrun[0] = false;
  }
  if ( stream_.mode != OUTPUT && handle->xrun[1] == true ) {
    status |= RTAUDIO_INPUT_OVERFLOW;
    handle->xrun[1] = false;
  }
  doStopStream = callback( stream_.userBuffer[0], stream_.userBuffer[1],
                           stream_.bufferSize, streamTime, status, stream_.callbackInfo.userData );
  if ( doStopStream == 2 ) {
    this->abortStream();
    return;
  }

  MUTEX_LOCK( &stream_.mutex );

  // The state might change while waiting on a mutex.
  if ( stream_.state == STREAM_STOPPED ) goto unlock;

  int result;
  char *buffer;
  int samples;
  RtAudioFormat format;

  if ( stream_.mode == OUTPUT || stream_.mode == DUPLEX ) {

    // Setup parameters and do buffer conversion if necessary.
    if ( stream_.doConvertBuffer[0] ) {
      buffer = stream_.deviceBuffer;
      convertBuffer( buffer, stream_.userBuffer[0], stream_.convertInfo[0] );
      samples = stream_.bufferSize * stream_.nDeviceChannels[0];
      format = stream_.deviceFormat[0];
    }
    else {
      buffer = stream_.userBuffer[0];
      samples = stream_.bufferSize * stream_.nUserChannels[0];
      format = stream_.userFormat;
    }

    // Do byte swapping if necessary.
    if ( stream_.doByteSwap[0] )
      byteSwapBuffer( buffer, samples, format );

    if ( stream_.mode == DUPLEX && handle->triggered == false ) {
      int trig = 0;
      ioctl( handle->id[0], SNDCTL_DSP_SETTRIGGER, &trig );
      result = write( handle->id[0], buffer, samples * formatBytes(format) );
      trig = PCM_ENABLE_INPUT|PCM_ENABLE_OUTPUT;
      ioctl( handle->id[0], SNDCTL_DSP_SETTRIGGER, &trig );
      handle->triggered = true;
    }
    else
      // Write samples to device.
      result = write( handle->id[0], buffer, samples * formatBytes(format) );

    if ( result == -1 ) {
      // We'll assume this is an underrun, though there isn't a
      // specific means for determining that.
      handle->xrun[0] = true;
      errorText_ = "RtApiOss::callbackEvent: audio write error.";
      error( RtAudioError::WARNING );
      // Continue on to input section.
    }
  }

  if ( stream_.mode == INPUT || stream_.mode == DUPLEX ) {

    // Setup parameters.
    if ( stream_.doConvertBuffer[1] ) {
      buffer = stream_.deviceBuffer;
      samples = stream_.bufferSize * stream_.nDeviceChannels[1];
      format = stream_.deviceFormat[1];
    }
    else {
      buffer = stream_.userBuffer[1];
      samples = stream_.bufferSize * stream_.nUserChannels[1];
      format = stream_.userFormat;
    }

    // Read samples from device.
    result = read( handle->id[1], buffer, samples * formatBytes(format) );

    if ( result == -1 ) {
      // We'll assume this is an overrun, though there isn't a
      // specific means for determining that.
      handle->xrun[1] = true;
      errorText_ = "RtApiOss::callbackEvent: audio read error.";
      error( RtAudioError::WARNING );
      goto unlock;
    }

    // Do byte swapping if necessary.
    if ( stream_.doByteSwap[1] )
      byteSwapBuffer( buffer, samples, format );

    // Do buffer conversion if necessary.
    if ( stream_.doConvertBuffer[1] )
      convertBuffer( stream_.userBuffer[1], stream_.deviceBuffer, stream_.convertInfo[1] );
  }

 unlock:
  MUTEX_UNLOCK( &stream_.mutex );

  RtApi::tickStreamTime();
  if ( doStopStream == 1 ) this->stopStream();
}

static void *ossCallbackHandler( void *ptr )
{
  CallbackInfo *info = (CallbackInfo *) ptr;
  RtApiOss *object = (RtApiOss *) info->object;
  bool *isRunning = &info->isRunning;

  while ( *isRunning == true ) {
    pthread_testcancel();
    object->callbackEvent();
  }

  pthread_exit( NULL );
}

//******************** End of __LINUX_OSS__ *********************//
#endif


// *************************************************** //
//
// Protected common (OS-independent) RtAudio methods.
//
// *************************************************** //

// This method can be modified to control the behavior of error
// message printing.
void RtApi :: error( RtAudioError::Type type )
{
  errorStream_.str(""); // clear the ostringstream

  RtAudioErrorCallback errorCallback = (RtAudioErrorCallback) stream_.callbackInfo.errorCallback;
  if ( errorCallback ) {
    // abortStream() can generate new error messages. Ignore them. Just keep original one.

    if ( firstErrorOccurred_ )
      return;

    firstErrorOccurred_ = true;
    const std::string errorMessage = errorText_;

    if ( type != RtAudioError::WARNING && stream_.state != STREAM_STOPPED) {
      stream_.callbackInfo.isRunning = false; // exit from the thread
      abortStream();
    }

    errorCallback( type, errorMessage );
    firstErrorOccurred_ = false;
    return;
  }

  if ( type == RtAudioError::WARNING && showWarnings_ == true )
    std::cerr << '\n' << errorText_ << "\n\n";
  // Modification: handled exceptions are not available in UE.
  // else if ( type != RtAudioError::WARNING )
	  // throw_wrapper( RtAudioError( errorText_, type ) );
}

void RtApi :: verifyStream()
{
  if ( stream_.state == STREAM_CLOSED ) {
    errorText_ = "RtApi:: a stream is not open!";
    error( RtAudioError::INVALID_USE );
  }
}

void RtApi :: clearStreamInfo()
{
  stream_.mode = UNINITIALIZED;
  stream_.state = STREAM_CLOSED;
  stream_.sampleRate = 0;
  stream_.bufferSize = 0;
  stream_.nBuffers = 0;
  stream_.userFormat = 0;
  stream_.userInterleaved = true;
  stream_.streamTime = 0.0;
  stream_.apiHandle = 0;
  stream_.deviceBuffer = 0;
  stream_.callbackInfo.callback = 0;
  stream_.callbackInfo.userData = 0;
  stream_.callbackInfo.isRunning = false;
  stream_.callbackInfo.errorCallback = 0;
  for ( int i=0; i<2; i++ ) {
    stream_.device[i] = 11111;
    stream_.doConvertBuffer[i] = false;
    stream_.deviceInterleaved[i] = true;
    stream_.doByteSwap[i] = false;
    stream_.nUserChannels[i] = 0;
    stream_.nDeviceChannels[i] = 0;
    stream_.channelOffset[i] = 0;
    stream_.deviceFormat[i] = 0;
    stream_.latency[i] = 0;
    stream_.userBuffer[i] = 0;
    stream_.convertInfo[i].channels = 0;
    stream_.convertInfo[i].inJump = 0;
    stream_.convertInfo[i].outJump = 0;
    stream_.convertInfo[i].inFormat = 0;
    stream_.convertInfo[i].outFormat = 0;
    stream_.convertInfo[i].inOffset.clear();
    stream_.convertInfo[i].outOffset.clear();
  }
}

unsigned int RtApi :: formatBytes( RtAudioFormat format )
{
  if ( format == RTAUDIO_SINT16 )
    return 2;
  else if ( format == RTAUDIO_SINT32 || format == RTAUDIO_FLOAT32 )
    return 4;
  else if ( format == RTAUDIO_FLOAT64 )
    return 8;
  else if ( format == RTAUDIO_SINT24 )
    return 3;
  else if ( format == RTAUDIO_SINT8 )
    return 1;

  errorText_ = "RtApi::formatBytes: undefined format.";
  error( RtAudioError::WARNING );

  return 0;
}

void RtApi :: setConvertInfo( StreamMode mode, unsigned int firstChannel )
{
  if ( mode == INPUT ) { // convert device to user buffer
    stream_.convertInfo[mode].inJump = stream_.nDeviceChannels[1];
    stream_.convertInfo[mode].outJump = stream_.nUserChannels[1];
    stream_.convertInfo[mode].inFormat = stream_.deviceFormat[1];
    stream_.convertInfo[mode].outFormat = stream_.userFormat;
  }
  else { // convert user to device buffer
    stream_.convertInfo[mode].inJump = stream_.nUserChannels[0];
    stream_.convertInfo[mode].outJump = stream_.nDeviceChannels[0];
    stream_.convertInfo[mode].inFormat = stream_.userFormat;
    stream_.convertInfo[mode].outFormat = stream_.deviceFormat[0];
  }

  if ( stream_.convertInfo[mode].inJump < stream_.convertInfo[mode].outJump )
    stream_.convertInfo[mode].channels = stream_.convertInfo[mode].inJump;
  else
    stream_.convertInfo[mode].channels = stream_.convertInfo[mode].outJump;

  // Set up the interleave/deinterleave offsets.
  if ( stream_.deviceInterleaved[mode] != stream_.userInterleaved ) {
    if ( ( mode == OUTPUT && stream_.deviceInterleaved[mode] ) ||
         ( mode == INPUT && stream_.userInterleaved ) ) {
      for ( int k=0; k<stream_.convertInfo[mode].channels; k++ ) {
        stream_.convertInfo[mode].inOffset.push_back( k * stream_.bufferSize );
        stream_.convertInfo[mode].outOffset.push_back( k );
        stream_.convertInfo[mode].inJump = 1;
      }
    }
    else {
      for ( int k=0; k<stream_.convertInfo[mode].channels; k++ ) {
        stream_.convertInfo[mode].inOffset.push_back( k );
        stream_.convertInfo[mode].outOffset.push_back( k * stream_.bufferSize );
        stream_.convertInfo[mode].outJump = 1;
      }
    }
  }
  else { // no (de)interleaving
    if ( stream_.userInterleaved ) {
      for ( int k=0; k<stream_.convertInfo[mode].channels; k++ ) {
        stream_.convertInfo[mode].inOffset.push_back( k );
        stream_.convertInfo[mode].outOffset.push_back( k );
      }
    }
    else {
      for ( int k=0; k<stream_.convertInfo[mode].channels; k++ ) {
        stream_.convertInfo[mode].inOffset.push_back( k * stream_.bufferSize );
        stream_.convertInfo[mode].outOffset.push_back( k * stream_.bufferSize );
        stream_.convertInfo[mode].inJump = 1;
        stream_.convertInfo[mode].outJump = 1;
      }
    }
  }

  // Add channel offset.
  if ( firstChannel > 0 ) {
    if ( stream_.deviceInterleaved[mode] ) {
      if ( mode == OUTPUT ) {
        for ( int k=0; k<stream_.convertInfo[mode].channels; k++ )
          stream_.convertInfo[mode].outOffset[k] += firstChannel;
      }
      else {
        for ( int k=0; k<stream_.convertInfo[mode].channels; k++ )
          stream_.convertInfo[mode].inOffset[k] += firstChannel;
      }
    }
    else {
      if ( mode == OUTPUT ) {
        for ( int k=0; k<stream_.convertInfo[mode].channels; k++ )
          stream_.convertInfo[mode].outOffset[k] += ( firstChannel * stream_.bufferSize );
      }
      else {
        for ( int k=0; k<stream_.convertInfo[mode].channels; k++ )
          stream_.convertInfo[mode].inOffset[k] += ( firstChannel  * stream_.bufferSize );
      }
    }
  }
}

void RtApi :: convertBuffer( char *outBuffer, char *inBuffer, ConvertInfo &info )
{
  // This function does format conversion, input/output channel compensation, and
  // data interleaving/deinterleaving.  24-bit integers are assumed to occupy
  // the lower three bytes of a 32-bit integer.

  // Clear our device buffer when in/out duplex device channels are different
  if ( outBuffer == stream_.deviceBuffer && stream_.mode == DUPLEX &&
       ( stream_.nDeviceChannels[0] < stream_.nDeviceChannels[1] ) )
    memset( outBuffer, 0, stream_.bufferSize * info.outJump * formatBytes( info.outFormat ) );

  int j;
  if (info.outFormat == RTAUDIO_FLOAT64) {
    Float64 scale;
    Float64 *out = (Float64 *)outBuffer;

    if (info.inFormat == RTAUDIO_SINT8) {
      signed char *in = (signed char *)inBuffer;
      scale = 1.0 / 127.5;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float64) in[info.inOffset[j]];
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT16) {
      Int16 *in = (Int16 *)inBuffer;
      scale = 1.0 / 32767.5;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float64) in[info.inOffset[j]];
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT24) {
      Int24 *in = (Int24 *)inBuffer;
      scale = 1.0 / 8388607.5;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float64) (in[info.inOffset[j]].asInt());
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT32) {
      Int32 *in = (Int32 *)inBuffer;
      scale = 1.0 / 2147483647.5;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float64) in[info.inOffset[j]];
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT32) {
      Float32 *in = (Float32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float64) in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT64) {
      // Channel compensation and/or (de)interleaving only.
      Float64 *in = (Float64 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
  }
  else if (info.outFormat == RTAUDIO_FLOAT32) {
    Float32 scale;
    Float32 *out = (Float32 *)outBuffer;

    if (info.inFormat == RTAUDIO_SINT8) {
      signed char *in = (signed char *)inBuffer;
      scale = (Float32) ( 1.0 / 127.5 );
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float32) in[info.inOffset[j]];
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT16) {
      Int16 *in = (Int16 *)inBuffer;
      scale = (Float32) ( 1.0 / 32767.5 );
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float32) in[info.inOffset[j]];
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT24) {
      Int24 *in = (Int24 *)inBuffer;
      scale = (Float32) ( 1.0 / 8388607.5 );
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float32) (in[info.inOffset[j]].asInt());
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT32) {
      Int32 *in = (Int32 *)inBuffer;
      scale = (Float32) ( 1.0 / 2147483647.5 );
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float32) in[info.inOffset[j]];
          out[info.outOffset[j]] += 0.5;
          out[info.outOffset[j]] *= scale;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT32) {
      // Channel compensation and/or (de)interleaving only.
      Float32 *in = (Float32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT64) {
      Float64 *in = (Float64 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Float32) in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
  }
  else if (info.outFormat == RTAUDIO_SINT32) {
    Int32 *out = (Int32 *)outBuffer;
    if (info.inFormat == RTAUDIO_SINT8) {
      signed char *in = (signed char *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) in[info.inOffset[j]];
          out[info.outOffset[j]] <<= 24;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT16) {
      Int16 *in = (Int16 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) in[info.inOffset[j]];
          out[info.outOffset[j]] <<= 16;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT24) {
      Int24 *in = (Int24 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) in[info.inOffset[j]].asInt();
          out[info.outOffset[j]] <<= 8;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT32) {
      // Channel compensation and/or (de)interleaving only.
      Int32 *in = (Int32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT32) {
      Float32 *in = (Float32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) (in[info.inOffset[j]] * 2147483647.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT64) {
      Float64 *in = (Float64 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) (in[info.inOffset[j]] * 2147483647.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
  }
  else if (info.outFormat == RTAUDIO_SINT24) {
    Int24 *out = (Int24 *)outBuffer;
    if (info.inFormat == RTAUDIO_SINT8) {
      signed char *in = (signed char *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) (in[info.inOffset[j]] << 16);
          //out[info.outOffset[j]] <<= 16;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT16) {
      Int16 *in = (Int16 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) (in[info.inOffset[j]] << 8);
          //out[info.outOffset[j]] <<= 8;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT24) {
      // Channel compensation and/or (de)interleaving only.
      Int24 *in = (Int24 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT32) {
      Int32 *in = (Int32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) (in[info.inOffset[j]] >> 8);
          //out[info.outOffset[j]] >>= 8;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT32) {
      Float32 *in = (Float32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) (in[info.inOffset[j]] * 8388607.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT64) {
      Float64 *in = (Float64 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int32) (in[info.inOffset[j]] * 8388607.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
  }
  else if (info.outFormat == RTAUDIO_SINT16) {
    Int16 *out = (Int16 *)outBuffer;
    if (info.inFormat == RTAUDIO_SINT8) {
      signed char *in = (signed char *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int16) in[info.inOffset[j]];
          out[info.outOffset[j]] <<= 8;
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT16) {
      // Channel compensation and/or (de)interleaving only.
      Int16 *in = (Int16 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT24) {
      Int24 *in = (Int24 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int16) (in[info.inOffset[j]].asInt() >> 8);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT32) {
      Int32 *in = (Int32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int16) ((in[info.inOffset[j]] >> 16) & 0x0000ffff);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT32) {
      Float32 *in = (Float32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int16) (in[info.inOffset[j]] * 32767.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT64) {
      Float64 *in = (Float64 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (Int16) (in[info.inOffset[j]] * 32767.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
  }
  else if (info.outFormat == RTAUDIO_SINT8) {
    signed char *out = (signed char *)outBuffer;
    if (info.inFormat == RTAUDIO_SINT8) {
      // Channel compensation and/or (de)interleaving only.
      signed char *in = (signed char *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = in[info.inOffset[j]];
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    if (info.inFormat == RTAUDIO_SINT16) {
      Int16 *in = (Int16 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (signed char) ((in[info.inOffset[j]] >> 8) & 0x00ff);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT24) {
      Int24 *in = (Int24 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (signed char) (in[info.inOffset[j]].asInt() >> 16);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_SINT32) {
      Int32 *in = (Int32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (signed char) ((in[info.inOffset[j]] >> 24) & 0x000000ff);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT32) {
      Float32 *in = (Float32 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (signed char) (in[info.inOffset[j]] * 127.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
    else if (info.inFormat == RTAUDIO_FLOAT64) {
      Float64 *in = (Float64 *)inBuffer;
      for (unsigned int i=0; i<stream_.bufferSize; i++) {
        for (j=0; j<info.channels; j++) {
          out[info.outOffset[j]] = (signed char) (in[info.inOffset[j]] * 127.5 - 0.5);
        }
        in += info.inJump;
        out += info.outJump;
      }
    }
  }
}

//static inline uint16_t bswap_16(uint16_t x) { return (x>>8) | (x<<8); }
//static inline uint32_t bswap_32(uint32_t x) { return (bswap_16(x&0xffff)<<16) | (bswap_16(x>>16)); }
//static inline uint64_t bswap_64(uint64_t x) { return (((unsigned long long)bswap_32(x&0xffffffffull))<<32) | (bswap_32(x>>32)); }

void RtApi :: byteSwapBuffer( char *buffer, unsigned int samples, RtAudioFormat format )
{
  char val;
  char *ptr;

  ptr = buffer;
  if ( format == RTAUDIO_SINT16 ) {
    for ( unsigned int i=0; i<samples; i++ ) {
      // Swap 1st and 2nd bytes.
      val = *(ptr);
      *(ptr) = *(ptr+1);
      *(ptr+1) = val;

      // Increment 2 bytes.
      ptr += 2;
    }
  }
  else if ( format == RTAUDIO_SINT32 ||
            format == RTAUDIO_FLOAT32 ) {
    for ( unsigned int i=0; i<samples; i++ ) {
      // Swap 1st and 4th bytes.
      val = *(ptr);
      *(ptr) = *(ptr+3);
      *(ptr+3) = val;

      // Swap 2nd and 3rd bytes.
      ptr += 1;
      val = *(ptr);
      *(ptr) = *(ptr+1);
      *(ptr+1) = val;

      // Increment 3 more bytes.
      ptr += 3;
    }
  }
  else if ( format == RTAUDIO_SINT24 ) {
    for ( unsigned int i=0; i<samples; i++ ) {
      // Swap 1st and 3rd bytes.
      val = *(ptr);
      *(ptr) = *(ptr+2);
      *(ptr+2) = val;

      // Increment 2 more bytes.
      ptr += 2;
    }
  }
  else if ( format == RTAUDIO_FLOAT64 ) {
    for ( unsigned int i=0; i<samples; i++ ) {
      // Swap 1st and 8th bytes
      val = *(ptr);
      *(ptr) = *(ptr+7);
      *(ptr+7) = val;

      // Swap 2nd and 7th bytes
      ptr += 1;
      val = *(ptr);
      *(ptr) = *(ptr+5);
      *(ptr+5) = val;

      // Swap 3rd and 6th bytes
      ptr += 1;
      val = *(ptr);
      *(ptr) = *(ptr+3);
      *(ptr+3) = val;

      // Swap 4th and 5th bytes
      ptr += 1;
      val = *(ptr);
      *(ptr) = *(ptr+1);
      *(ptr+1) = val;

      // Increment 5 more bytes.
      ptr += 5;
    }
  }
}

  // Indentation settings for Vim and Emacs
  //
  // Local Variables:
  // c-basic-offset: 2
  // indent-tabs-mode: nil
  // End:
  //
  // vim: et sts=2 sw=2

#endif // PLATFORM_WINDOWS

#endif // WITH_RTAUDIO
