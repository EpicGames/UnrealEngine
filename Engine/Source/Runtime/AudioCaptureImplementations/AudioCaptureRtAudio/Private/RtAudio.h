// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioCaptureCore.h"

#if PLATFORM_MICROSOFT || PLATFORM_MAC
/************************************************************************/
/*! \class RtAudio
    \brief Realtime audio i/o C++ classes.

    RtAudio provides a common API (Application Programming Interface)
    for realtime audio input/output across Linux (native ALSA, Jack,
    and OSS), Macintosh OS X (CoreAudio and Jack), and Windows
    (DirectSound, ASIO and WASAPI) operating systems.

    RtAudio WWW site: http://www.music.mcgill.ca/~gary/rtaudio/

    RtAudio: realtime audio i/o C++ classes
    Copyright (c) 2001-2017 Gary P. Scavone

    Permission is hereby granted, free of charge, to any person
    obtaining a copy of this software and associated documentation files
    (the "Software"), to deal in the Software without restriction,
    including without limitation the rights to use, copy, modify, merge,
    publish, distribute, sublicense, and/or sell copies of the Software,
    and to permit persons to whom the Software is furnished to do so,
    subject to the following conditions:

    The above copyright notice and this permission notice shall be
    included in all copies or substantial portions of the Software.

    Any person wishing to distribute modifications to the Software is
    asked to send the modifications to the original developer so that
    they can be incorporated into the canonical version.  This is,
    however, not a binding provision of this license.

    THE SOFTWARE IS PROVIDED "AS IS", WITHOUT WARRANTY OF ANY KIND,
    EXPRESS OR IMPLIED, INCLUDING BUT NOT LIMITED TO THE WARRANTIES OF
    MERCHANTABILITY, FITNESS FOR A PARTICULAR PURPOSE AND NONINFRINGEMENT.
    IN NO EVENT SHALL THE AUTHORS OR COPYRIGHT HOLDERS BE LIABLE FOR
    ANY CLAIM, DAMAGES OR OTHER LIABILITY, WHETHER IN AN ACTION OF
    CONTRACT, TORT OR OTHERWISE, ARISING FROM, OUT OF OR IN CONNECTION
    WITH THE SOFTWARE OR THE USE OR OTHER DEALINGS IN THE SOFTWARE.
*/
/************************************************************************/

/*!
  \file RtAudio.h
 */

#ifndef __RTAUDIO_H
#define __RTAUDIO_H

// Per-platform preprocessor defines here.
#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"

#ifndef __WINDOWS_DS__
#define __WINDOWS_DS__
#endif // __WINDOWS_DS__

#ifdef __WINDOWS_WASAPI__
#undef __WINDOWS_WASAPI__
#endif

#elif PLATFORM_MICROSOFT

// On Xbox, we use WASAPI.
#include "Windows/WindowsHWrapper.h"

#ifdef __WINDOWS_DS__
#undef __WINDOWS_DS__
#endif // __WINDOWS_DS__

#ifndef __WINDOWS_WASAPI__
#define __WINDOWS_WASAPI__
#endif

#elif PLATFORM_MAC
#ifndef __MACOSX_CORE__
#define __MACOSX_CORE__
#endif // __MACOSX_CORE__
#endif // PLATFORM_WINDOWS

#define RTAUDIO_VERSION "5.0.0"

#if defined _WIN32 || defined __CYGWIN__
  #define RTAUDIO_DLL_PUBLIC
#else
  #if __GNUC__ >= 4
    #define RTAUDIO_DLL_PUBLIC __attribute__( (visibility( "default" )) )
  #else
    #define RTAUDIO_DLL_PUBLIC
  #endif
#endif

THIRD_PARTY_INCLUDES_START
#include <string>
#include <vector>
#include <stdexcept>
#include <iostream>
THIRD_PARTY_INCLUDES_END

/*! \typedef typedef unsigned long RtAudioFormat;
    \brief RtAudio data format type.

    Support for signed integers and floats.  Audio data fed to/from an
    RtAudio stream is assumed to ALWAYS be in host byte order.  The
    internal routines will automatically take care of any necessary
    byte-swapping between the host format and the soundcard.  Thus,
    endian-ness is not a concern in the following format definitions.

    - \e RTAUDIO_SINT8:   8-bit signed integer.
    - \e RTAUDIO_SINT16:  16-bit signed integer.
    - \e RTAUDIO_SINT24:  24-bit signed integer.
    - \e RTAUDIO_SINT32:  32-bit signed integer.
    - \e RTAUDIO_FLOAT32: Normalized between plus/minus 1.0.
    - \e RTAUDIO_FLOAT64: Normalized between plus/minus 1.0.
*/
typedef unsigned long RtAudioFormat;
static const RtAudioFormat RTAUDIO_SINT8 = 0x1;    // 8-bit signed integer.
static const RtAudioFormat RTAUDIO_SINT16 = 0x2;   // 16-bit signed integer.
static const RtAudioFormat RTAUDIO_SINT24 = 0x4;   // 24-bit signed integer.
static const RtAudioFormat RTAUDIO_SINT32 = 0x8;   // 32-bit signed integer.
static const RtAudioFormat RTAUDIO_FLOAT32 = 0x10; // Normalized between plus/minus 1.0.
static const RtAudioFormat RTAUDIO_FLOAT64 = 0x20; // Normalized between plus/minus 1.0.

/*! \typedef typedef unsigned long RtAudioStreamFlags;
    \brief RtAudio stream option flags.

    The following flags can be OR'ed together to allow a client to
    make changes to the default stream behavior:

    - \e RTAUDIO_NONINTERLEAVED:   Use non-interleaved buffers (default = interleaved).
    - \e RTAUDIO_MINIMIZE_LATENCY: Attempt to set stream parameters for lowest possible latency.
    - \e RTAUDIO_HOG_DEVICE:       Attempt grab device for exclusive use.
    - \e RTAUDIO_ALSA_USE_DEFAULT: Use the "default" PCM device (ALSA only).
    - \e RTAUDIO_JACK_DONT_CONNECT: Do not automatically connect ports (JACK only).

    By default, RtAudio streams pass and receive audio data from the
    client in an interleaved format.  By passing the
    RTAUDIO_NONINTERLEAVED flag to the openStream() function, audio
    data will instead be presented in non-interleaved buffers.  In
    this case, each buffer argument in the RtAudioCallback function
    will point to a single array of data, with \c nFrames samples for
    each channel concatenated back-to-back.  For example, the first
    sample of data for the second channel would be located at index \c
    nFrames (assuming the \c buffer pointer was recast to the correct
    data type for the stream).

    Certain audio APIs offer a number of parameters that influence the
    I/O latency of a stream.  By default, RtAudio will attempt to set
    these parameters internally for robust (glitch-free) performance
    (though some APIs, like Windows Direct Sound, make this difficult).
    By passing the RTAUDIO_MINIMIZE_LATENCY flag to the openStream()
    function, internal stream settings will be influenced in an attempt
    to minimize stream latency, though possibly at the expense of stream
    performance.

    If the RTAUDIO_HOG_DEVICE flag is set, RtAudio will attempt to
    open the input and/or output stream device(s) for exclusive use.
    Note that this is not possible with all supported audio APIs.

    If the RTAUDIO_SCHEDULE_REALTIME flag is set, RtAudio will attempt 
    to select realtime scheduling (round-robin) for the callback thread.

    If the RTAUDIO_ALSA_USE_DEFAULT flag is set, RtAudio will attempt to
    open the "default" PCM device when using the ALSA API. Note that this
    will override any specified input or output device id.

    If the RTAUDIO_JACK_DONT_CONNECT flag is set, RtAudio will not attempt
    to automatically connect the ports of the client to the audio device.
*/
typedef unsigned int RtAudioStreamFlags;
static const RtAudioStreamFlags RTAUDIO_NONINTERLEAVED = 0x1;    // Use non-interleaved buffers (default = interleaved).
static const RtAudioStreamFlags RTAUDIO_MINIMIZE_LATENCY = 0x2;  // Attempt to set stream parameters for lowest possible latency.
static const RtAudioStreamFlags RTAUDIO_HOG_DEVICE = 0x4;        // Attempt grab device and prevent use by others.
static const RtAudioStreamFlags RTAUDIO_SCHEDULE_REALTIME = 0x8; // Try to select realtime scheduling for callback thread.
static const RtAudioStreamFlags RTAUDIO_ALSA_USE_DEFAULT = 0x10; // Use the "default" PCM device (ALSA only).
static const RtAudioStreamFlags RTAUDIO_JACK_DONT_CONNECT = 0x20; // Do not automatically connect ports (JACK only).

/*! \typedef typedef unsigned long RtAudioStreamStatus;
    \brief RtAudio stream status (over- or underflow) flags.

    Notification of a stream over- or underflow is indicated by a
    non-zero stream \c status argument in the RtAudioCallback function.
    The stream status can be one of the following two options,
    depending on whether the stream is open for output and/or input:

    - \e RTAUDIO_INPUT_OVERFLOW:   Input data was discarded because of an overflow condition at the driver.
    - \e RTAUDIO_OUTPUT_UNDERFLOW: The output buffer ran low, likely producing a break in the output sound.
*/
typedef unsigned int RtAudioStreamStatus;
static const RtAudioStreamStatus RTAUDIO_INPUT_OVERFLOW = 0x1;    // Input data was discarded because of an overflow condition at the driver.
static const RtAudioStreamStatus RTAUDIO_OUTPUT_UNDERFLOW = 0x2;  // The output buffer ran low, likely causing a gap in the output sound.

//! RtAudio callback function prototype.
/*!
   All RtAudio clients must create a function of type RtAudioCallback
   to read and/or write data from/to the audio stream.  When the
   underlying audio system is ready for new input or output data, this
   function will be invoked.

   \param outputBuffer For output (or duplex) streams, the client
          should write \c nFrames of audio sample frames into this
          buffer.  This argument should be recast to the datatype
          specified when the stream was opened.  For input-only
          streams, this argument will be NULL.

   \param inputBuffer For input (or duplex) streams, this buffer will
          hold \c nFrames of input audio sample frames.  This
          argument should be recast to the datatype specified when the
          stream was opened.  For output-only streams, this argument
          will be NULL.

   \param nFrames The number of sample frames of input or output
          data in the buffers.  The actual buffer size in bytes is
          dependent on the data type and number of channels in use.

   \param streamTime The number of seconds that have elapsed since the
          stream was started.

   \param status If non-zero, this argument indicates a data overflow
          or underflow condition for the stream.  The particular
          condition can be determined by comparison with the
          RtAudioStreamStatus flags.

   \param userData A pointer to optional data provided by the client
          when opening the stream (default = NULL).

   To continue normal stream operation, the RtAudioCallback function
   should return a value of zero.  To stop the stream and drain the
   output buffer, the function should return a value of one.  To abort
   the stream immediately, the client should return a value of two.
 */
typedef int (*RtAudioCallback)( void *outputBuffer, void *inputBuffer,
                                unsigned int nFrames,
                                double streamTime,
                                RtAudioStreamStatus status,
                                void *userData );

/************************************************************************/
/*! \class RtAudioError
    \brief Exception handling class for RtAudio.

    The RtAudioError class is quite simple but it does allow errors to be
    "caught" by RtAudioError::Type. See the RtAudio documentation to know
    which methods can throw an RtAudioError.
*/
/************************************************************************/

class RtAudioError : public std::runtime_error
{
 public:
  //! Defined RtAudioError types.
  enum Type {
    WARNING,           /*!< A non-critical error. */
    DEBUG_WARNING,     /*!< A non-critical error which might be useful for debugging. */
    UNSPECIFIED,       /*!< The default, unspecified error type. */
    NO_DEVICES_FOUND,  /*!< No devices found on system. */
    INVALID_DEVICE,    /*!< An invalid device ID was specified. */
    MEMORY_ERROR,      /*!< An error occured during memory allocation. */
    INVALID_PARAMETER, /*!< An invalid parameter was specified to a function. */
    INVALID_USE,       /*!< The function was called incorrectly. */
    DRIVER_ERROR,      /*!< A system driver error occured. */
    SYSTEM_ERROR,      /*!< A system error occured. */
    THREAD_ERROR       /*!< A thread error occured. */
  };

  //! The constructor.
  RtAudioError( const std::string& message,
                Type type = RtAudioError::UNSPECIFIED )
    : std::runtime_error(message), type_(type) {}

  //! Prints thrown error message to stderr.
  virtual void printMessage( void ) const
    { std::cerr << '\n' << what() << "\n\n"; }

  //! Returns the thrown error message type.
  virtual const Type& getType(void) const { return type_; }

  //! Returns the thrown error message string.
  virtual const std::string getMessage(void) const
    { return std::string(what()); }

 protected:
  Type type_;
};

//! RtAudio error callback function prototype.
/*!
    \param type Type of error.
    \param errorText Error description.
 */
typedef void (*RtAudioErrorCallback)( RtAudioError::Type type, const std::string &errorText );

// **************************************************************** //
//
// RtAudio class declaration.
//
// RtAudio is a "controller" used to select an available audio i/o
// interface.  It presents a common API for the user to call but all
// functionality is implemented by the class RtApi and its
// subclasses.  RtAudio creates an instance of an RtApi subclass
// based on the user's API choice.  If no choice is made, RtAudio
// attempts to make a "logical" API selection.
//
// **************************************************************** //

class RtApi;

class RtAudio
{
 public:

  //! Audio API specifier arguments.
  enum Api {
    UNSPECIFIED,    /*!< Search for a working compiled API. */
    LINUX_ALSA,     /*!< The Advanced Linux Sound Architecture API. */
    LINUX_PULSE,    /*!< The Linux PulseAudio API. */
    LINUX_OSS,      /*!< The Linux Open Sound System API. */
    UNIX_JACK,      /*!< The Jack Low-Latency Audio Server API. */
    MACOSX_CORE,    /*!< Macintosh OS-X Core Audio API. */
    WINDOWS_WASAPI, /*!< The Microsoft WASAPI API. */
    WINDOWS_ASIO,   /*!< The Steinberg Audio Stream I/O API. */
    WINDOWS_DS,     /*!< The Microsoft Direct Sound API. */
    RTAUDIO_DUMMY   /*!< A compilable but non-functional API. */
  };

  //! The public device information structure for returning queried values.
  struct DeviceInfo {
    bool probed;                  /*!< true if the device capabilities were successfully probed. */
	std::string name;             /*!< Character string device display name. */
	std::string deviceId;               /*!< Character string device identifier. */
    unsigned int outputChannels;  /*!< Maximum output channels supported by device. */
    unsigned int inputChannels;   /*!< Maximum input channels supported by device. */
    unsigned int duplexChannels;  /*!< Maximum simultaneous input/output channels supported by device. */
    bool isDefaultOutput;         /*!< true if this is the default output device. */
    bool isDefaultInput;          /*!< true if this is the default input device. */
    std::vector<unsigned int> sampleRates; /*!< Supported sample rates (queried from list of standard rates). */
    unsigned int preferredSampleRate; /*!< Preferred sample rate, eg. for WASAPI the system sample rate. */
    RtAudioFormat nativeFormats;  /*!< Bit mask of supported data formats. */

    // Default constructor.
    DeviceInfo()
      :probed(false), outputChannels(0), inputChannels(0), duplexChannels(0),
       isDefaultOutput(false), isDefaultInput(false), preferredSampleRate(0), nativeFormats(0) {}
  };

  //! The structure for specifying input or ouput stream parameters.
  struct StreamParameters {
    unsigned int deviceId;     /*!< Device index (0 to getDeviceCount() - 1). */
    unsigned int nChannels;    /*!< Number of channels. */
    unsigned int firstChannel; /*!< First channel index on device (default = 0). */

    // Default constructor.
    StreamParameters()
      : deviceId(0), nChannels(0), firstChannel(0) {}
  };

  //! The structure for specifying stream options.
  /*!
    The following flags can be OR'ed together to allow a client to
    make changes to the default stream behavior:

    - \e RTAUDIO_NONINTERLEAVED:    Use non-interleaved buffers (default = interleaved).
    - \e RTAUDIO_MINIMIZE_LATENCY:  Attempt to set stream parameters for lowest possible latency.
    - \e RTAUDIO_HOG_DEVICE:        Attempt grab device for exclusive use.
    - \e RTAUDIO_SCHEDULE_REALTIME: Attempt to select realtime scheduling for callback thread.
    - \e RTAUDIO_ALSA_USE_DEFAULT:  Use the "default" PCM device (ALSA only).

    By default, RtAudio streams pass and receive audio data from the
    client in an interleaved format.  By passing the
    RTAUDIO_NONINTERLEAVED flag to the openStream() function, audio
    data will instead be presented in non-interleaved buffers.  In
    this case, each buffer argument in the RtAudioCallback function
    will point to a single array of data, with \c nFrames samples for
    each channel concatenated back-to-back.  For example, the first
    sample of data for the second channel would be located at index \c
    nFrames (assuming the \c buffer pointer was recast to the correct
    data type for the stream).

    Certain audio APIs offer a number of parameters that influence the
    I/O latency of a stream.  By default, RtAudio will attempt to set
    these parameters internally for robust (glitch-free) performance
    (though some APIs, like Windows Direct Sound, make this difficult).
    By passing the RTAUDIO_MINIMIZE_LATENCY flag to the openStream()
    function, internal stream settings will be influenced in an attempt
    to minimize stream latency, though possibly at the expense of stream
    performance.

    If the RTAUDIO_HOG_DEVICE flag is set, RtAudio will attempt to
    open the input and/or output stream device(s) for exclusive use.
    Note that this is not possible with all supported audio APIs.

    If the RTAUDIO_SCHEDULE_REALTIME flag is set, RtAudio will attempt 
    to select realtime scheduling (round-robin) for the callback thread.
    The \c priority parameter will only be used if the RTAUDIO_SCHEDULE_REALTIME
    flag is set. It defines the thread's realtime priority.

    If the RTAUDIO_ALSA_USE_DEFAULT flag is set, RtAudio will attempt to
    open the "default" PCM device when using the ALSA API. Note that this
    will override any specified input or output device id.

    The \c numberOfBuffers parameter can be used to control stream
    latency in the Windows DirectSound, Linux OSS, and Linux Alsa APIs
    only.  A value of two is usually the smallest allowed.  Larger
    numbers can potentially result in more robust stream performance,
    though likely at the cost of stream latency.  The value set by the
    user is replaced during execution of the RtAudio::openStream()
    function by the value actually used by the system.

    The \c streamName parameter can be used to set the client name
    when using the Jack API.  By default, the client name is set to
    RtApiJack.  However, if you wish to create multiple instances of
    RtAudio with Jack, each instance must have a unique client name.
  */
  struct StreamOptions {
    RtAudioStreamFlags flags;      /*!< A bit-mask of stream flags (RTAUDIO_NONINTERLEAVED, RTAUDIO_MINIMIZE_LATENCY, RTAUDIO_HOG_DEVICE, RTAUDIO_ALSA_USE_DEFAULT). */
    unsigned int numberOfBuffers;  /*!< Number of stream buffers. */
    std::string streamName;        /*!< A stream name (currently used only in Jack). */
    int priority;                  /*!< Scheduling priority of callback thread (only used with flag RTAUDIO_SCHEDULE_REALTIME). */

    // Default constructor.
    StreamOptions()
    : flags(0), numberOfBuffers(0), priority(0) {}
  };

  //! A static function to determine the current RtAudio version.
  static AUDIOCAPTURERTAUDIO_API std::string getVersion( void );

  //! A static function to determine the available compiled audio APIs.
  /*!
    The values returned in the std::vector can be compared against
    the enumerated list values.  Note that there can be more than one
    API compiled for certain operating systems.
  */
  static AUDIOCAPTURERTAUDIO_API void getCompiledApi( std::vector<RtAudio::Api> &apis );

  //! The class constructor.
  /*!
    The constructor performs minor initialization tasks.  An exception
    can be thrown if no API support is compiled.

    If no API argument is specified and multiple API support has been
    compiled, the default order of use is JACK, ALSA, OSS (Linux
    systems) and ASIO, DS (Windows systems).
  */
  AUDIOCAPTURERTAUDIO_API RtAudio( RtAudio::Api api=UNSPECIFIED );

  //! The destructor.
  /*!
    If a stream is running or open, it will be stopped and closed
    automatically.
  */
  AUDIOCAPTURERTAUDIO_API ~RtAudio();

  //! Returns the audio API specifier for the current instance of RtAudio.
  AUDIOCAPTURERTAUDIO_API RtAudio::Api getCurrentApi( void );

  //! A public function that queries for the number of audio devices available.
  /*!
    This function performs a system query of available devices each time it
    is called, thus supporting devices connected \e after instantiation. If
    a system error occurs during processing, a warning will be issued. 
  */
  AUDIOCAPTURERTAUDIO_API unsigned int getDeviceCount( void );

  //! Return an RtAudio::DeviceInfo structure for a specified device number.
  /*!

    Any device integer between 0 and getDeviceCount() - 1 is valid.
    If an invalid argument is provided, an RtAudioError (type = INVALID_USE)
    will be thrown.  If a device is busy or otherwise unavailable, the
    structure member "probed" will have a value of "false" and all
    other members are undefined.  If the specified device is the
    current default input or output device, the corresponding
    "isDefault" member will have a value of "true".
  */
  AUDIOCAPTURERTAUDIO_API RtAudio::DeviceInfo getDeviceInfo( unsigned int device );

  //! A function that returns the index of the default output device.
  /*!
    If the underlying audio API does not provide a "default
    device", or if no devices are available, the return value will be
    0.  Note that this is a valid device identifier and it is the
    client's responsibility to verify that a device is available
    before attempting to open a stream.
  */
  AUDIOCAPTURERTAUDIO_API unsigned int getDefaultOutputDevice( void );

  //! A function that returns the index of the default input device.
  /*!
    If the underlying audio API does not provide a "default
    device", or if no devices are available, the return value will be
    0.  Note that this is a valid device identifier and it is the
    client's responsibility to verify that a device is available
    before attempting to open a stream.
  */
  AUDIOCAPTURERTAUDIO_API unsigned int getDefaultInputDevice( void );

  //! A public function for opening a stream with the specified parameters.
  /*!
    An RtAudioError (type = SYSTEM_ERROR) is thrown if a stream cannot be
    opened with the specified parameters or an error occurs during
    processing.  An RtAudioError (type = INVALID_USE) is thrown if any
    invalid device ID or channel number parameters are specified.

    \param outputParameters Specifies output stream parameters to use
           when opening a stream, including a device ID, number of channels,
           and starting channel number.  For input-only streams, this
           argument should be NULL.  The device ID is an index value between
           0 and getDeviceCount() - 1.
    \param inputParameters Specifies input stream parameters to use
           when opening a stream, including a device ID, number of channels,
           and starting channel number.  For output-only streams, this
           argument should be NULL.  The device ID is an index value between
           0 and getDeviceCount() - 1.
    \param format An RtAudioFormat specifying the desired sample data format.
    \param sampleRate The desired sample rate (sample frames per second).
    \param *bufferFrames A pointer to a value indicating the desired
           internal buffer size in sample frames.  The actual value
           used by the device is returned via the same pointer.  A
           value of zero can be specified, in which case the lowest
           allowable value is determined.
    \param callback A client-defined function that will be invoked
           when input data is available and/or output data is needed.
    \param userData An optional pointer to data that can be accessed
           from within the callback function.
    \param options An optional pointer to a structure containing various
           global stream options, including a list of OR'ed RtAudioStreamFlags
           and a suggested number of stream buffers that can be used to 
           control stream latency.  More buffers typically result in more
           robust performance, though at a cost of greater latency.  If a
           value of zero is specified, a system-specific median value is
           chosen.  If the RTAUDIO_MINIMIZE_LATENCY flag bit is set, the
           lowest allowable value is used.  The actual value used is
           returned via the structure argument.  The parameter is API dependent.
    \param errorCallback A client-defined function that will be invoked
           when an error has occured.
  */
  AUDIOCAPTURERTAUDIO_API void openStream( RtAudio::StreamParameters *outputParameters,
                   RtAudio::StreamParameters *inputParameters,
                   RtAudioFormat format, unsigned int sampleRate,
                   unsigned int *bufferFrames, RtAudioCallback callback,
                   void *userData = NULL, RtAudio::StreamOptions *options = NULL, RtAudioErrorCallback errorCallback = NULL );

  //! A function that closes a stream and frees any associated stream memory.
  /*!
    If a stream is not open, this function issues a warning and
    returns (no exception is thrown).
  */
  AUDIOCAPTURERTAUDIO_API void closeStream( void );

  //! A function that starts a stream.
  /*!
    An RtAudioError (type = SYSTEM_ERROR) is thrown if an error occurs
    during processing.  An RtAudioError (type = INVALID_USE) is thrown if a
    stream is not open.  A warning is issued if the stream is already
    running.
  */
  AUDIOCAPTURERTAUDIO_API void startStream( void );

  //! Stop a stream, allowing any samples remaining in the output queue to be played.
  /*!
    An RtAudioError (type = SYSTEM_ERROR) is thrown if an error occurs
    during processing.  An RtAudioError (type = INVALID_USE) is thrown if a
    stream is not open.  A warning is issued if the stream is already
    stopped.
  */
  AUDIOCAPTURERTAUDIO_API void stopStream( void );

  //! Stop a stream, discarding any samples remaining in the input/output queue.
  /*!
    An RtAudioError (type = SYSTEM_ERROR) is thrown if an error occurs
    during processing.  An RtAudioError (type = INVALID_USE) is thrown if a
    stream is not open.  A warning is issued if the stream is already
    stopped.
  */
  AUDIOCAPTURERTAUDIO_API void abortStream( void );

  //! Returns true if a stream is open and false if not.
  AUDIOCAPTURERTAUDIO_API bool isStreamOpen( void ) const;

  //! Returns true if the stream is running and false if it is stopped or not open.
  AUDIOCAPTURERTAUDIO_API bool isStreamRunning( void ) const;

  //! Returns the number of elapsed seconds since the stream was started.
  /*!
    If a stream is not open, an RtAudioError (type = INVALID_USE) will be thrown.
  */
  AUDIOCAPTURERTAUDIO_API double getStreamTime( void );

  //! Set the stream time to a time in seconds greater than or equal to 0.0.
  /*!
    If a stream is not open, an RtAudioError (type = INVALID_USE) will be thrown.
  */
  AUDIOCAPTURERTAUDIO_API void setStreamTime( double time );

  //! Returns the internal stream latency in sample frames.
  /*!
    The stream latency refers to delay in audio input and/or output
    caused by internal buffering by the audio system and/or hardware.
    For duplex streams, the returned value will represent the sum of
    the input and output latencies.  If a stream is not open, an
    RtAudioError (type = INVALID_USE) will be thrown.  If the API does not
    report latency, the return value will be zero.
  */
  AUDIOCAPTURERTAUDIO_API long getStreamLatency( void );

 //! Returns actual sample rate in use by the stream.
 /*!
   On some systems, the sample rate used may be slightly different
   than that specified in the stream parameters.  If a stream is not
   open, an RtAudioError (type = INVALID_USE) will be thrown.
 */
  AUDIOCAPTURERTAUDIO_API unsigned int getStreamSampleRate( void );

  //! Specify whether warning messages should be printed to stderr.
  AUDIOCAPTURERTAUDIO_API void showWarnings( bool value = true );

 protected:

  AUDIOCAPTURERTAUDIO_API void openRtApi( RtAudio::Api api );
  RtApi *rtapi_;
};

// Operating system dependent thread functionality.
#if defined(__WINDOWS_DS__) || defined(__WINDOWS_ASIO__) || defined(__WINDOWS_WASAPI__)

  #ifndef NOMINMAX
    #define NOMINMAX
  #endif
  #include <windows.h>
  #include <process.h>

  typedef uintptr_t ThreadHandle;
  typedef CRITICAL_SECTION StreamMutex;

#elif defined(__LINUX_ALSA__) || defined(__LINUX_PULSE__) || defined(__UNIX_JACK__) || defined(__LINUX_OSS__) || defined(__MACOSX_CORE__)
  // Using pthread library for various flavors of unix.
  #include <pthread.h>

  typedef pthread_t ThreadHandle;
  typedef pthread_mutex_t StreamMutex;

#else // Setup for "dummy" behavior

  #define __RTAUDIO_DUMMY__
  typedef int ThreadHandle;
  typedef int StreamMutex;

#endif

// This global structure type is used to pass callback information
// between the private RtAudio stream structure and global callback
// handling functions.
struct CallbackInfo {
  void *object;    // Used as a "this" pointer.
  ThreadHandle thread;
  void *callback;
  void *userData;
  void *errorCallback;
  void *apiInfo;   // void pointer for API specific callback information
  bool isRunning;
  bool doRealtime;
  int priority;

  // Default constructor.
  CallbackInfo()
  :object(0), callback(0), userData(0), errorCallback(0), apiInfo(0), isRunning(false), doRealtime(false), priority(0) {}
};

// **************************************************************** //
//
// RtApi class declaration.
//
// Subclasses of RtApi contain all API- and OS-specific code necessary
// to fully implement the RtAudio API.
//
// Note that RtApi is an abstract base class and cannot be
// explicitly instantiated.  The class RtAudio will create an
// instance of an RtApi subclass (RtApiOss, RtApiAlsa,
// RtApiJack, RtApiCore, RtApiDs, or RtApiAsio).
//
// **************************************************************** //

#pragma pack(push, 1)
class S24 {

 protected:
  unsigned char c3[3];

 public:
  S24() {}

  S24& operator = ( const int& i ) {
    c3[0] = (i & 0x000000ff);
    c3[1] = (i & 0x0000ff00) >> 8;
    c3[2] = (i & 0x00ff0000) >> 16;
    return *this;
  }

  S24( const S24& v ) { *this = v; }
  S24( const double& d ) { *this = (int) d; }
  S24( const float& f ) { *this = (int) f; }
  S24( const signed short& s ) { *this = (int) s; }
  S24( const char& c ) { *this = (int) c; }

  int asInt() {
    int i = c3[0] | (c3[1] << 8) | (c3[2] << 16);
    if (i & 0x800000) i |= ~0xffffff;
    return i;
  }
};
#pragma pack(pop)

#if defined( HAVE_GETTIMEOFDAY )
  #include <sys/time.h>
#endif

#include <sstream>

class RtApi
{
public:

  AUDIOCAPTURERTAUDIO_API RtApi();
  AUDIOCAPTURERTAUDIO_API virtual ~RtApi();
  virtual RtAudio::Api getCurrentApi( void ) = 0;
  virtual unsigned int getDeviceCount( void ) = 0;
  virtual RtAudio::DeviceInfo getDeviceInfo( unsigned int device ) = 0;
  AUDIOCAPTURERTAUDIO_API virtual unsigned int getDefaultInputDevice( void );
  AUDIOCAPTURERTAUDIO_API virtual unsigned int getDefaultOutputDevice( void );
  AUDIOCAPTURERTAUDIO_API void openStream( RtAudio::StreamParameters *outputParameters,
                   RtAudio::StreamParameters *inputParameters,
                   RtAudioFormat format, unsigned int sampleRate,
                   unsigned int *bufferFrames, RtAudioCallback callback,
                   void *userData, RtAudio::StreamOptions *options,
                   RtAudioErrorCallback errorCallback );
  AUDIOCAPTURERTAUDIO_API virtual void closeStream( void );
  virtual void startStream( void ) = 0;
  virtual void stopStream( void ) = 0;
  virtual void abortStream( void ) = 0;
  AUDIOCAPTURERTAUDIO_API long getStreamLatency( void );
  AUDIOCAPTURERTAUDIO_API unsigned int getStreamSampleRate( void );
  AUDIOCAPTURERTAUDIO_API virtual double getStreamTime( void );
  AUDIOCAPTURERTAUDIO_API virtual void setStreamTime( double time );
  bool isStreamOpen( void ) const { return stream_.state != STREAM_CLOSED; }
  bool isStreamRunning( void ) const { return stream_.state == STREAM_RUNNING; }
  void showWarnings( bool value ) { showWarnings_ = value; }


protected:

  static AUDIOCAPTURERTAUDIO_API const unsigned int MAX_SAMPLE_RATES;
  static AUDIOCAPTURERTAUDIO_API const unsigned int SAMPLE_RATES[];

  enum { FAILURE, SUCCESS };

  enum StreamState {
    STREAM_STOPPED,
    STREAM_STOPPING,
    STREAM_RUNNING,
    STREAM_CLOSED = -50
  };

  enum StreamMode {
    OUTPUT,
    INPUT,
    DUPLEX,
    UNINITIALIZED = -75
  };

  // A protected structure used for buffer conversion.
  struct ConvertInfo {
    int channels;
    int inJump, outJump;
    RtAudioFormat inFormat, outFormat;
    std::vector<int> inOffset;
    std::vector<int> outOffset;
  };

  // A protected structure for audio streams.
  struct RtApiStream {
    unsigned int device[2];    // Playback and record, respectively.
    void *apiHandle;           // void pointer for API specific stream handle information
    StreamMode mode;           // OUTPUT, INPUT, or DUPLEX.
    StreamState state;         // STOPPED, RUNNING, or CLOSED
    char *userBuffer[2];       // Playback and record, respectively.
    char *deviceBuffer;
    bool doConvertBuffer[2];   // Playback and record, respectively.
    bool userInterleaved;
    bool deviceInterleaved[2]; // Playback and record, respectively.
    bool doByteSwap[2];        // Playback and record, respectively.
    unsigned int sampleRate;
    unsigned int bufferSize;
    unsigned int nBuffers;
    unsigned int nUserChannels[2];    // Playback and record, respectively.
    unsigned int nDeviceChannels[2];  // Playback and record channels, respectively.
    unsigned int channelOffset[2];    // Playback and record, respectively.
    unsigned long latency[2];         // Playback and record, respectively.
    RtAudioFormat userFormat;
    RtAudioFormat deviceFormat[2];    // Playback and record, respectively.
    StreamMutex mutex;
    CallbackInfo callbackInfo;
    ConvertInfo convertInfo[2];
    double streamTime;         // Number of elapsed seconds since the stream started.

#if defined(HAVE_GETTIMEOFDAY)
    struct timeval lastTickTimestamp;
#endif

    RtApiStream()
      :apiHandle(0), deviceBuffer(0) { device[0] = 11111; device[1] = 11111; }
  };

  typedef S24 Int24;
  typedef signed short Int16;
  typedef signed int Int32;
  typedef float Float32;
  typedef double Float64;

  std::ostringstream errorStream_;
  std::string errorText_;
  bool showWarnings_;
  RtApiStream stream_;
  bool firstErrorOccurred_;

  /*!
    Protected, api-specific method that attempts to open a device
    with the given parameters.  This function MUST be implemented by
    all subclasses.  If an error is encountered during the probe, a
    "warning" message is reported and FAILURE is returned. A
    successful probe is indicated by a return value of SUCCESS.
  */
  AUDIOCAPTURERTAUDIO_API virtual bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels, 
                                unsigned int firstChannel, unsigned int sampleRate,
                                RtAudioFormat format, unsigned int *bufferSize,
                                RtAudio::StreamOptions *options );

  //! A protected function used to increment the stream time.
  AUDIOCAPTURERTAUDIO_API void tickStreamTime( void );

  //! Protected common method to clear an RtApiStream structure.
  AUDIOCAPTURERTAUDIO_API void clearStreamInfo();

  /*!
    Protected common method that throws an RtAudioError (type =
    INVALID_USE) if a stream is not open.
  */
  AUDIOCAPTURERTAUDIO_API void verifyStream( void );

  //! Protected common error method to allow global control over error handling.
  AUDIOCAPTURERTAUDIO_API void error( RtAudioError::Type type );

  /*!
    Protected method used to perform format, channel number, and/or interleaving
    conversions between the user and device buffers.
  */
  AUDIOCAPTURERTAUDIO_API void convertBuffer( char *outBuffer, char *inBuffer, ConvertInfo &info );

  //! Protected common method used to perform byte-swapping on buffers.
  AUDIOCAPTURERTAUDIO_API void byteSwapBuffer( char *buffer, unsigned int samples, RtAudioFormat format );

  //! Protected common method that returns the number of bytes for a given format.
  AUDIOCAPTURERTAUDIO_API unsigned int formatBytes( RtAudioFormat format );

  //! Protected common method that sets up the parameters for buffer conversion.
  AUDIOCAPTURERTAUDIO_API void setConvertInfo( StreamMode mode, unsigned int firstChannel );
};

// **************************************************************** //
//
// Inline RtAudio definitions.
//
// **************************************************************** //

inline RtAudio::Api RtAudio :: getCurrentApi( void ) { return rtapi_->getCurrentApi(); }
inline unsigned int RtAudio :: getDeviceCount( void ) { return rtapi_->getDeviceCount(); }
inline RtAudio::DeviceInfo RtAudio :: getDeviceInfo( unsigned int device ) { return rtapi_->getDeviceInfo( device ); }
inline unsigned int RtAudio :: getDefaultInputDevice( void ) { return rtapi_->getDefaultInputDevice(); }
inline unsigned int RtAudio :: getDefaultOutputDevice( void ) { return rtapi_->getDefaultOutputDevice(); }
inline void RtAudio :: closeStream( void ) { return rtapi_->closeStream(); }
inline void RtAudio :: startStream( void ) { return rtapi_->startStream(); }
inline void RtAudio :: stopStream( void )  { return rtapi_->stopStream(); }
inline void RtAudio :: abortStream( void ) { return rtapi_->abortStream(); }
inline bool RtAudio :: isStreamOpen( void ) const { return rtapi_->isStreamOpen(); }
inline bool RtAudio :: isStreamRunning( void ) const { return rtapi_->isStreamRunning(); }
inline long RtAudio :: getStreamLatency( void ) { return rtapi_->getStreamLatency(); }
inline unsigned int RtAudio :: getStreamSampleRate( void ) { return rtapi_->getStreamSampleRate(); }
inline double RtAudio :: getStreamTime( void ) { return rtapi_->getStreamTime(); }
inline void RtAudio :: setStreamTime( double time ) { return rtapi_->setStreamTime( time ); }
inline void RtAudio :: showWarnings( bool value ) { rtapi_->showWarnings( value ); }

// RtApi Subclass prototypes.

#if defined(__MACOSX_CORE__)

#include <CoreAudio/AudioHardware.h>

class RtApiCore: public RtApi
{
public:

  RtApiCore();
  ~RtApiCore();
  RtAudio::Api getCurrentApi( void ) { return RtAudio::MACOSX_CORE; }
  unsigned int getDeviceCount( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  unsigned int getDefaultOutputDevice( void );
  unsigned int getDefaultInputDevice( void );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );
  long getStreamLatency( void );

  // This function is intended for internal use only.  It must be
  // public because it is called by the internal callback handler,
  // which is not a member of RtAudio.  External use of this function
  // will most likely produce highly undesireable results!
  bool callbackEvent( AudioDeviceID deviceId,
                      const AudioBufferList *inBufferList,
                      const AudioBufferList *outBufferList );

  private:

  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels, 
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int *bufferSize,
                        RtAudio::StreamOptions *options );
  static const char* getErrorCode( OSStatus code );
};

#endif

#if defined(__UNIX_JACK__)

class RtApiJack: public RtApi
{
public:

  RtApiJack();
  ~RtApiJack();
  RtAudio::Api getCurrentApi( void ) { return RtAudio::UNIX_JACK; }
  unsigned int getDeviceCount( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );
  long getStreamLatency( void );

  // This function is intended for internal use only.  It must be
  // public because it is called by the internal callback handler,
  // which is not a member of RtAudio.  External use of this function
  // will most likely produce highly undesireable results!
  bool callbackEvent( unsigned long nframes );

  private:

  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels, 
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int *bufferSize,
                        RtAudio::StreamOptions *options );

  bool shouldAutoconnect_;
};

#endif

#if defined(__WINDOWS_ASIO__)

class RtApiAsio: public RtApi
{
public:

  RtApiAsio();
  ~RtApiAsio();
  RtAudio::Api getCurrentApi( void ) { return RtAudio::WINDOWS_ASIO; }
  unsigned int getDeviceCount( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );
  long getStreamLatency( void );

  // This function is intended for internal use only.  It must be
  // public because it is called by the internal callback handler,
  // which is not a member of RtAudio.  External use of this function
  // will most likely produce highly undesireable results!
  bool callbackEvent( long bufferIndex );

  private:

  std::vector<RtAudio::DeviceInfo> devices_;
  void saveDeviceInfo( void );
  bool coInitialized_;
  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels, 
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int *bufferSize,
                        RtAudio::StreamOptions *options );
};

#endif

#if defined(__WINDOWS_DS__)

class RtApiDs: public RtApi
{
public:

  RtApiDs();
  ~RtApiDs();
  RtAudio::Api getCurrentApi( void ) { return RtAudio::WINDOWS_DS; }
  unsigned int getDeviceCount( void );
  unsigned int getDefaultOutputDevice( void );
  unsigned int getDefaultInputDevice( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );
  long getStreamLatency( void );

  // This function is intended for internal use only.  It must be
  // public because it is called by the internal callback handler,
  // which is not a member of RtAudio.  External use of this function
  // will most likely produce highly undesireable results!
  void callbackEvent( void );

  private:

  bool coInitialized_;
  bool buffersRolling;
  long duplexPrerollBytes;
  std::vector<struct DsDevice> dsDevices;
  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels, 
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int *bufferSize,
                        RtAudio::StreamOptions *options );
};

#endif

#if defined(__WINDOWS_WASAPI__)

struct IMMDeviceEnumerator;

class RtApiWasapi : public RtApi
{
public:
  RtApiWasapi();
  ~RtApiWasapi();

  RtAudio::Api getCurrentApi( void ) { return RtAudio::WINDOWS_WASAPI; }
  unsigned int getDeviceCount( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  unsigned int getDefaultOutputDevice( void );
  unsigned int getDefaultInputDevice( void );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );

private:
  bool coInitialized_;
  IMMDeviceEnumerator* deviceEnumerator_;

  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int* bufferSize,
                        RtAudio::StreamOptions* options );

  static DWORD WINAPI runWasapiThread( void* wasapiPtr );
  static DWORD WINAPI stopWasapiThread( void* wasapiPtr );
  static DWORD WINAPI abortWasapiThread( void* wasapiPtr );
  void wasapiThread();
};

#endif

#if defined(__LINUX_ALSA__)

class RtApiAlsa: public RtApi
{
public:

  RtApiAlsa();
  ~RtApiAlsa();
  RtAudio::Api getCurrentApi() { return RtAudio::LINUX_ALSA; }
  unsigned int getDeviceCount( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );

  // This function is intended for internal use only.  It must be
  // public because it is called by the internal callback handler,
  // which is not a member of RtAudio.  External use of this function
  // will most likely produce highly undesireable results!
  void callbackEvent( void );

  private:

  std::vector<RtAudio::DeviceInfo> devices_;
  void saveDeviceInfo( void );
  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels, 
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int *bufferSize,
                        RtAudio::StreamOptions *options );
};

#endif

#if defined(__LINUX_PULSE__)

class RtApiPulse: public RtApi
{
public:
  ~RtApiPulse();
  RtAudio::Api getCurrentApi() { return RtAudio::LINUX_PULSE; }
  unsigned int getDeviceCount( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );

  // This function is intended for internal use only.  It must be
  // public because it is called by the internal callback handler,
  // which is not a member of RtAudio.  External use of this function
  // will most likely produce highly undesireable results!
  void callbackEvent( void );

  private:

  std::vector<RtAudio::DeviceInfo> devices_;
  void saveDeviceInfo( void );
  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels,
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int *bufferSize,
                        RtAudio::StreamOptions *options );
};

#endif

#if defined(__LINUX_OSS__)

class RtApiOss: public RtApi
{
public:

  RtApiOss();
  ~RtApiOss();
  RtAudio::Api getCurrentApi() { return RtAudio::LINUX_OSS; }
  unsigned int getDeviceCount( void );
  RtAudio::DeviceInfo getDeviceInfo( unsigned int device );
  void closeStream( void );
  void startStream( void );
  void stopStream( void );
  void abortStream( void );

  // This function is intended for internal use only.  It must be
  // public because it is called by the internal callback handler,
  // which is not a member of RtAudio.  External use of this function
  // will most likely produce highly undesireable results!
  void callbackEvent( void );

  private:

  bool probeDeviceOpen( unsigned int device, StreamMode mode, unsigned int channels, 
                        unsigned int firstChannel, unsigned int sampleRate,
                        RtAudioFormat format, unsigned int *bufferSize,
                        RtAudio::StreamOptions *options );
};

#endif

#if defined(__RTAUDIO_DUMMY__)

class RtApiDummy: public RtApi
{
public:

  RtApiDummy() { errorText_ = "RtApiDummy: This class provides no functionality."; error( RtAudioError::WARNING ); }
  RtAudio::Api getCurrentApi( void ) { return RtAudio::RTAUDIO_DUMMY; }
  unsigned int getDeviceCount( void ) { return 0; }
  RtAudio::DeviceInfo getDeviceInfo( unsigned int /*device*/ ) { RtAudio::DeviceInfo info; return info; }
  void closeStream( void ) {}
  void startStream( void ) {}
  void stopStream( void ) {}
  void abortStream( void ) {}

  private:

  bool probeDeviceOpen( unsigned int /*device*/, StreamMode /*mode*/, unsigned int /*channels*/, 
                        unsigned int /*firstChannel*/, unsigned int /*sampleRate*/,
                        RtAudioFormat /*format*/, unsigned int * /*bufferSize*/,
                        RtAudio::StreamOptions * /*options*/ ) { return false; }
};

#endif

#endif

// Indentation settings for Vim and Emacs
//
// Local Variables:
// c-basic-offset: 2
// indent-tabs-mode: nil
// End:
//
// vim: et sts=2 sw=2
#endif // PLATFORM_WINDOWS
