// Copyright Epic Games, Inc. All Rights Reserved.

#include "RtAudioWrapper.h"
#include "AudioCaptureCoreLog.h"

#if PLATFORM_MICROSOFT || PLATFORM_MAC

#include "RtAudio.h"
#include "Templates/PimplPtr.h"
#include "Containers/UnrealString.h"

THIRD_PARTY_INCLUDES_START
#include <stdexcept>
#include <string>
THIRD_PARTY_INCLUDES_END

namespace Audio
{
	FRtAudioInputWrapper::FRtAudioInputWrapper()
	: Impl(MakePimpl<RtAudio>())
	{
	}

	// Returns ID of default input device
	uint32 FRtAudioInputWrapper::GetDefaultInputDevice()
	{
		static const TCHAR* Action = TEXT("get default input device");
		try
		{
			return Impl->getDefaultInputDevice();
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		}

		return InvalidDeviceID;
	}

	// Returns info about device.
	FRtAudioInputWrapper::FDeviceInfo FRtAudioInputWrapper::GetDeviceInfo(uint32 InDeviceID)
	{
		static const TCHAR* Action = TEXT("get device info");
		try
		{
			RtAudio::DeviceInfo RtInfo = Impl->getDeviceInfo(InDeviceID);
			return FDeviceInfo {static_cast<float>(RtInfo.preferredSampleRate), RtInfo.inputChannels};
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		}
		return FDeviceInfo{};
	}

	// Suppress deprecation warning for use of FAudioCallback
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Opens an audio stream. Returns true on success, false on error. 
	bool FRtAudioInputWrapper::OpenStream(const FStreamParameters& InStreamParams, float InDesiredSampleRate, uint32* InOutDesiredBufferNumFrames, FAudioCallback Callback, void* InUserData)
	{
		static const TCHAR* Action = TEXT("open RtAudio stream");
		try
		{
			RtAudio::StreamParameters RtStreamParams;
			RtStreamParams.deviceId = InStreamParams.DeviceID; 
			RtStreamParams.nChannels = InStreamParams.NumChannels;
			RtStreamParams.firstChannel = 0;

			Impl->openStream(nullptr, &RtStreamParams, RTAUDIO_SINT16, InDesiredSampleRate, InOutDesiredBufferNumFrames, Callback, InUserData);
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
			return false;
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
			return false;
		}

		return true;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Starts an open stream
	void FRtAudioInputWrapper::StartStream()
	{
		static const TCHAR* Action = TEXT("start RtAudio stream");
		try
		{
			Impl->startStream();
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		}
	}

	// Returns true if the stream is open.
	bool FRtAudioInputWrapper::IsStreamOpen()
	{
		static const TCHAR* Action = TEXT("check if RtAudio stream is open");
		try
		{
			return Impl->isStreamOpen();
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		}

		return false;
	}

	// Stops stream, discarding any remaining samples
	void FRtAudioInputWrapper::AbortStream()
	{
		static const TCHAR* Action = TEXT("abort RtAudio stream");
		try
		{
			Impl->abortStream();
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		}
	}

	// Stops stream, allowing any remaining samples to be played.
	void FRtAudioInputWrapper::StopStream()
	{
		static const TCHAR* Action = TEXT("stop RtAudio stream");
		try
		{
			Impl->stopStream();
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		}
	}

	// Close stream and free associated memory.
	void FRtAudioInputWrapper::CloseStream()
	{
		static const TCHAR* Action = TEXT("close RtAudio stream");
		try
		{
			Impl->closeStream();
		}
		catch (const std::exception& e)
		{
			FString Message(e.what());
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls: %ls.", Action, *Message);
		}
		catch (...)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "Failed to %ls.", Action);
		}
	}
}

#else

namespace Audio
{
	FRtAudioInputWrapper::FRtAudioInputWrapper()
	{
		UE_LOGF(LogAudioCaptureCore, Error, "RtAudio is not supported on this platform");
	}

	// Returns ID of default input device
	uint32 FRtAudioInputWrapper::GetDefaultInputDevice()
	{
		return InvalidDeviceID;
	}

	// Returns info about device.
	FRtAudioInputWrapper::FDeviceInfo FRtAudioInputWrapper::GetDeviceInfo(uint32 InDeviceID)
	{
		return FDeviceInfo{};
	}

	// Opens an audio stream. Returns true on success, false on error. 
	bool FRtAudioInputWrapper::OpenStream(const FStreamParameters& InStreamParams, float InDesiredSampleRate, uint32* InOutDesiredBufferNumFrames, FAudioCallback Callback, void* InUserData)
	{
		return false;
	}

	// Starts an open stream
	void FRtAudioInputWrapper::StartStream()
	{
	}

	// Returns true if the stream is open.
	bool FRtAudioInputWrapper::IsStreamOpen()
	{
		return false;
	}

	// Stops stream, discarding any remaining samples
	void FRtAudioInputWrapper::AbortStream()
	{
	}

	// Stops stream, allowing any remaining samples to be played.
	void FRtAudioInputWrapper::StopStream()
	{
	}

	// Close stream and free associated memory.
	void FRtAudioInputWrapper::CloseStream()
	{
	}
}

#endif // #if PLATFORM_MICROSOFT || PLATFORM_MAC
