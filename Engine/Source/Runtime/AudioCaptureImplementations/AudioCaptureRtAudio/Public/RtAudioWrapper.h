// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/PimplPtr.h"

class RtAudio; // Forward declare;

namespace Audio
{
	// FRtAudioInputWrapper wraps RtAudio implementation and provides a way to 
	// capture input audio from an hardware audio device. 
	class AUDIOCAPTURERTAUDIO_API UE_DEPRECATED(5.3, "FRtAudioInputWrapper is deprecated, please use FAudioCapture instead.") FRtAudioInputWrapper
	{
	public:
		static constexpr uint32 InvalidDeviceID = INDEX_NONE;

		// Callback function for receiving audio from RtAudio
		using FAudioCallback = int (*)(void *OutputBuffer, void *InputBuffer, unsigned int NumFrames, double StreamTime, uint32 StreamStatus, void *UserData);

		// Parameters used for opening streams
		struct FStreamParameters
		{
			uint32 DeviceID = 0;
			uint32 NumChannels = 0;
		};

		// Parameters describing audio input device.
		struct FDeviceInfo
		{
			float PreferredSampleRate = 0.f;
			uint32 NumChannels = 0;
		};

		FRtAudioInputWrapper();

		// Returns ID of default input device
		uint32 GetDefaultInputDevice();

		// Returns info about device.
		FDeviceInfo GetDeviceInfo(uint32 InDeviceID);

		// Opens an audio stream. Returns true on success, false on error. 
		bool OpenStream(const FStreamParameters& InStreamParams, float InDesiredSampleRate, uint32* InOutDesiredBufferNumFrames, FAudioCallback Callback, void* InUserData);

		// Returns true if the stream is open.
		bool IsStreamOpen();

		// Starts an open stream
		void StartStream();

		// Stops stream, discarding any remaining samples
		void AbortStream();

		// Stops stream, allowing any remaining samples to be played.
		void StopStream();

		// Close stream and free associated memory.
		void CloseStream();
	
	private:

		TPimplPtr<RtAudio> Impl;
	};
}

