// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioCaptureAudioUnit : ModuleRules
{
    public AudioCaptureAudioUnit(ReadOnlyTargetRules Target) : base(Target)
    {
	    bRequiresPlatformSDK = true;
        PublicFrameworks.AddRange(new string[] { "CoreAudio", "AVFoundation", "AudioToolbox" });
        PrivateDependencyModuleNames.Add("Core");
        PrivateDependencyModuleNames.Add("AudioCaptureCore");
        PublicDefinitions.Add("WITH_AUDIOCAPTURE=1");
    }
}
