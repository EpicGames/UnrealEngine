// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureAudioUnit.h"
#include "AudioCaptureCoreLog.h"
#include <AVFoundation/AVAudioSession.h>

const int32 kInputBus = 1;
const int32 kOutputBus = 0;
const int32 kRemoteIODeviceIndex = 0;
const int32 kVoiceProcessingIODeviceIndex = 1;

Audio::FAudioCaptureAudioUnitStream::FAudioCaptureAudioUnitStream()
	: NumChannels(0)
	, SampleRate(0)
	, bIsStreamOpen(false)
	, bHasCaptureStarted(false)
	, bIsHardwareVoiceProcessingSupported(false)
{
}

static OSStatus RecordingCallback(void *inRefCon,
	AudioUnitRenderActionFlags *ioActionFlags,
	const AudioTimeStamp *inTimeStamp,
	UInt32 inBusNumber,
	UInt32 inNumberFrames,
	AudioBufferList *ioData)
{
	Audio::FAudioCaptureAudioUnitStream* AudioCapture = (Audio::FAudioCaptureAudioUnitStream*)inRefCon;
	return AudioCapture->OnCaptureCallback(ioActionFlags, inTimeStamp, inBusNumber, inNumberFrames, ioData);
}

OSStatus Audio::FAudioCaptureAudioUnitStream::OnCaptureCallback(AudioUnitRenderActionFlags* ioActionFlags, const AudioTimeStamp* inTimeStamp, UInt32 inBusNumber, UInt32 inNumberFrames, AudioBufferList* ioData)
{
	OSStatus status = noErr;

	const int NeededBufferSize = inNumberFrames * NumChannels * sizeof(float);
	if (CaptureBuffer.Num() == 0 || BufferSize < NeededBufferSize)
	{
		BufferSize = NeededBufferSize;
		AllocateBuffer(BufferSize);
	}
	
	AudioBufferList* BufferList = (AudioBufferList*) CaptureBuffer.GetData();
	for (int i=0; i < BufferList->mNumberBuffers; ++i) {
		BufferList->mBuffers[i].mDataByteSize = (UInt32) BufferSize;
	}
	
	status = AudioUnitRender(IOUnit,
		ioActionFlags,
		inTimeStamp,
		inBusNumber,
		inNumberFrames,
		BufferList);
	check(status == noErr);

	void* InBuffer = (void*)BufferList->mBuffers[0].mData; // only first channel ?!
	OnAudioCapture(InBuffer, inNumberFrames, 0.0, false); // need calculate timestamp

	return noErr;
}

void Audio::FAudioCaptureAudioUnitStream::AllocateBuffer(int SizeInBytes)
{
	size_t NeedBytes = sizeof(AudioBufferList) + NumChannels * (sizeof(AudioBuffer) + SizeInBytes);

	CaptureBuffer.SetNum(NeedBytes);

	AudioBufferList* list = (AudioBufferList*) CaptureBuffer.GetData();
	uint8* data = CaptureBuffer.GetData() + sizeof(AudioBufferList) + sizeof(AudioBuffer);
	
	list->mNumberBuffers = NumChannels;
	for(int i=0; i < NumChannels; i++)
	{
		list->mBuffers[i].mNumberChannels = 1;
		list->mBuffers[i].mDataByteSize   = (UInt32) SizeInBytes;
		list->mBuffers[i].mData           = data;
		
		data += (SizeInBytes + sizeof(AudioBuffer));
	}
}

bool Audio::FAudioCaptureAudioUnitStream::GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex)
{
	switch (DeviceIndex)
	{
		case Audio::DefaultDeviceIndex:
		case kRemoteIODeviceIndex:
		{
			OutInfo.DeviceName = FString("Remote IO Audio Component");
			OutInfo.DeviceId = OutInfo.DeviceName;
			OutInfo.InputChannels = 1;
			OutInfo.PreferredSampleRate = 48000;
			OutInfo.bSupportsHardwareAEC = false;
			return true;
		}
		case kVoiceProcessingIODeviceIndex:
		{
			OutInfo.DeviceName = FString("VoiceProcesing IO Audio Component");
			OutInfo.DeviceId = OutInfo.DeviceName;
			OutInfo.InputChannels = 1;
			OutInfo.PreferredSampleRate = 48000;
			OutInfo.bSupportsHardwareAEC = true;
			return true;
		}
		default:
		{
			return false;
		}
	}
}

bool Audio::FAudioCaptureAudioUnitStream::OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired)
{
	switch (InParams.DeviceIndex)
	{
		case Audio::DefaultDeviceIndex:
		case kRemoteIODeviceIndex:
		{
			bIsHardwareVoiceProcessingSupported	= false;
			UE_CLOGF(InParams.bUseHardwareAEC, LogAudioCaptureCore, Warning, "Hardware support is only available for VoiceProcessing IO Audio Component (DeviceIndex = %d)", kVoiceProcessingIODeviceIndex);
			break;
		}
		case kVoiceProcessingIODeviceIndex:
		{
			bIsHardwareVoiceProcessingSupported	= true;
			break;
		}
		default:
		{
			return false;
		}
	}

	NumChannels = 1;
	SampleRate = 48000;
	OnCapture = MoveTemp(InOnCapture);

	OSStatus Status = noErr;
		
	// Source of info "Technical Note TN2091 - Device input using the HAL Output Audio Unit"
	
	AudioComponentDescription desc;
	desc.componentType = kAudioUnitType_Output;
	desc.componentSubType = bIsHardwareVoiceProcessingSupported ? kAudioUnitSubType_VoiceProcessingIO : kAudioUnitSubType_RemoteIO;
	desc.componentManufacturer = kAudioUnitManufacturer_Apple;
	desc.componentFlags = 0;
	desc.componentFlagsMask = 0;

	// Using VoiceProcessing IO may change the AVAudioSession mode to AVAudioSessionModeVoiceChat. Cache AVAudioSession settings and restore them after initialisation
	NSString* Mode = [[AVAudioSession sharedInstance] mode];
	NSString* Category = [[AVAudioSession sharedInstance] category];
	AVAudioSessionCategoryOptions Options = [[AVAudioSession sharedInstance] categoryOptions];

	AudioComponent InputComponent = AudioComponentFindNext(NULL, &desc);

	Status = AudioComponentInstanceNew(InputComponent, &IOUnit);
	check(Status == noErr);

	// Enable recording
	uint32 EnableIO = 1;
	Status = AudioUnitSetProperty(IOUnit,
		kAudioOutputUnitProperty_EnableIO,
		kAudioUnitScope_Input,
		kInputBus,
		&EnableIO,
		sizeof(EnableIO));
	check(Status == noErr);

	// Disable output part
	EnableIO = 0;
	Status = AudioUnitSetProperty(IOUnit,
		kAudioOutputUnitProperty_EnableIO,
		kAudioUnitScope_Output,
		kOutputBus,
		&EnableIO,
		sizeof(EnableIO));
	check(Status == noErr);

	AudioStreamBasicDescription StreamDescription = {0};
	const UInt32 BytesPerSample = sizeof(Float32);
	
	StreamDescription.mSampleRate       = SampleRate;
	StreamDescription.mFormatID         = kAudioFormatLinearPCM;
	StreamDescription.mFormatFlags      = kAudioFormatFlagsNativeFloatPacked;
	StreamDescription.mChannelsPerFrame = NumChannels;
	StreamDescription.mBitsPerChannel   = 8 * BytesPerSample;
	StreamDescription.mBytesPerFrame    = BytesPerSample * StreamDescription.mChannelsPerFrame;
	StreamDescription.mFramesPerPacket  = 1;
	StreamDescription.mBytesPerPacket   = StreamDescription.mFramesPerPacket * StreamDescription.mBytesPerFrame;
	
	// Configure output format
	Status = AudioUnitSetProperty(IOUnit,
		kAudioUnitProperty_StreamFormat,
		kAudioUnitScope_Output,
		kInputBus,
		&StreamDescription,
		sizeof(StreamDescription));
	check(Status == noErr);

	// Setup capture callback
	AURenderCallbackStruct CallbackInfo;
	CallbackInfo.inputProc = RecordingCallback;
	CallbackInfo.inputProcRefCon = this;
	Status = AudioUnitSetProperty(IOUnit,
		kAudioOutputUnitProperty_SetInputCallback,
		kAudioUnitScope_Global,
		kInputBus,
		&CallbackInfo,
		sizeof(CallbackInfo));
	check(Status == noErr);
	
	// Configure unit processing
	if (bIsHardwareVoiceProcessingSupported)
	{
		SetHardwareFeatureEnabled(Audio::EHardwareInputFeature::EchoCancellation, InParams.bUseHardwareAEC);
		SetHardwareFeatureEnabled(Audio::EHardwareInputFeature::AutomaticGainControl, InParams.bUseHardwareAEC);
	}

	// Initialize audio unit
	Status = AudioUnitInitialize(IOUnit);
	check(Status == noErr);

	bIsStreamOpen = (Status == noErr);

	if ([[[AVAudioSession sharedInstance] category] compare:Category] != NSOrderedSame ||
		[[[AVAudioSession sharedInstance] mode] compare:Mode] != NSOrderedSame ||
		[[AVAudioSession sharedInstance] categoryOptions] != Options)
	{
		NSError* ActiveError = nil;
		[[AVAudioSession sharedInstance] setCategory:Category mode:Mode options:Options error:&ActiveError];
	}

	return bIsStreamOpen;
}

bool Audio::FAudioCaptureAudioUnitStream::CloseStream()
{
	StopStream();
	AudioComponentInstanceDispose(IOUnit);
	bIsStreamOpen = false;

	return true;
}

bool Audio::FAudioCaptureAudioUnitStream::StartStream()
{
	bHasCaptureStarted = (AudioOutputUnitStart(IOUnit) == noErr);
	return bHasCaptureStarted;
}

bool Audio::FAudioCaptureAudioUnitStream::StopStream()
{
	bHasCaptureStarted = false;
	return (AudioOutputUnitStop(IOUnit) == noErr);
}

bool Audio::FAudioCaptureAudioUnitStream::AbortStream()
{
	StopStream();
	CloseStream();
	return true;
}

bool Audio::FAudioCaptureAudioUnitStream::GetStreamTime(double& OutStreamTime)
{
	OutStreamTime = 0.0f;
	return true;
}

bool Audio::FAudioCaptureAudioUnitStream::IsStreamOpen() const
{
	return bIsStreamOpen;
}

bool Audio::FAudioCaptureAudioUnitStream::IsCapturing() const
{
	return bHasCaptureStarted;
}

void Audio::FAudioCaptureAudioUnitStream::OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow)
{
	OnCapture(InBuffer, InBufferFrames, NumChannels, SampleRate, StreamTime, bOverflow);
}

bool Audio::FAudioCaptureAudioUnitStream::GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices)
{
	OutDevices.Reset();

	GetCaptureDeviceInfo(OutDevices.AddDefaulted_GetRef(), kRemoteIODeviceIndex);
	GetCaptureDeviceInfo(OutDevices.AddDefaulted_GetRef(), kVoiceProcessingIODeviceIndex);

	return true;
}

void Audio::FAudioCaptureAudioUnitStream::SetHardwareFeatureEnabled(EHardwareInputFeature FeatureType, bool bEnabled)
{
	if (IOUnit == nil || !bIsHardwareVoiceProcessingSupported)
	{
		UE_CLOGF(!bIsHardwareVoiceProcessingSupported, LogAudioCaptureCore, Warning, "Hardware support is only available for VoiceProcessing IO Audio Component (DeviceIndex = %d)", kVoiceProcessingIODeviceIndex);
		return;
	}

	// we ignore result those functions because sometime we can't set this parameters
	OSStatus status = noErr;

	switch(FeatureType)
	{
		case Audio::EHardwareInputFeature::EchoCancellation:
		{
			const UInt32 EnableParam = (bEnabled ? 0 : 1);
			status = AudioUnitSetProperty(IOUnit,
			  kAUVoiceIOProperty_BypassVoiceProcessing,
			  kAudioUnitScope_Global,
			  kInputBus,
			  &EnableParam,
			  sizeof(EnableParam)
			);
		}
			break;
		case Audio::EHardwareInputFeature::AutomaticGainControl:
		{
			const UInt32 EnableParam = (bEnabled ? 1 : 0);
			status = AudioUnitSetProperty(IOUnit,
			  kAUVoiceIOProperty_VoiceProcessingEnableAGC,
			  kAudioUnitScope_Global,
			  kInputBus,
			  &EnableParam,
			  sizeof(EnableParam)
			);
		}
			break;
	}
}
