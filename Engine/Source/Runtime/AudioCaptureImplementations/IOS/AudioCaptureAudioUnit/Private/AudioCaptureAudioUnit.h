// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioCaptureCore.h"
#include <AudioToolbox/AudioToolbox.h>
#include <AudioUnit/AudioUnit.h>

namespace Audio
{
	class FAudioCaptureAudioUnitStream : public IAudioCaptureStream
	{
	public:
		FAudioCaptureAudioUnitStream();

		virtual bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex) override;
		virtual bool OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired) override;
		virtual bool CloseStream() override;
		virtual bool StartStream() override;
		virtual bool StopStream() override;
		virtual bool AbortStream() override;
		virtual bool GetStreamTime(double& OutStreamTime) override;
		virtual int32 GetSampleRate() const override { return SampleRate; }
		virtual bool IsStreamOpen() const override;
		virtual bool IsCapturing() const override;
		virtual void OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow) override;
		virtual bool GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices) override;
		virtual void SetHardwareFeatureEnabled(EHardwareInputFeature FeatureType, bool bEnabled);

		OSStatus OnCaptureCallback( AudioUnitRenderActionFlags* ioActionFlags, const AudioTimeStamp* inTimeStamp, UInt32 inBusNumber, UInt32 inNumberFrames, AudioBufferList* ioData );

	private:
		void AllocateBuffer(int SizeInBytes);
		
		AudioComponentInstance IOUnit;
		
		FOnAudioCaptureFunction OnCapture;
		int32 NumChannels;
		int32 SampleRate;
		TArray<uint8> CaptureBuffer;
		int BufferSize = 0;
		bool bIsStreamOpen;
		bool bHasCaptureStarted;
		bool bIsHardwareVoiceProcessingSupported;
	};
}
