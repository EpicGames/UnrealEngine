// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AudioCaptureDeviceInterface.h"
#include "AudioCaptureAudioUnit.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"

namespace Audio
{
	class FAudioCaptureAudioUnitFactory : public IAudioCaptureFactory
	{
	public:
		virtual TUniquePtr<IAudioCaptureStream> CreateNewAudioCaptureStream() override
		{
			return TUniquePtr<IAudioCaptureStream>(new FAudioCaptureAudioUnitStream());
		}
	};

	class FAudioCaptureAudioUnitModule : public IModuleInterface
	{
	private:
		FAudioCaptureAudioUnitFactory AudioCaptureFactory;

	public:
		virtual void StartupModule() override
		{
			IModularFeatures::Get().RegisterModularFeature(IAudioCaptureFactory::GetModularFeatureName(), &AudioCaptureFactory);
		}

		virtual void ShutdownModule() override
		{
			IModularFeatures::Get().UnregisterModularFeature(IAudioCaptureFactory::GetModularFeatureName(), &AudioCaptureFactory);
		}
	};
}

IMPLEMENT_MODULE(Audio::FAudioCaptureAudioUnitModule, AudioCaptureAudioUnit)
