// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioCaptureWasapi : ModuleRules
{
	public AudioCaptureWasapi(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"AudioCaptureCore",
				"AudioPlatformSupportWasapi"
			}
		);

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PublicDefinitions.Add("WITH_AUDIOCAPTURE=1");
		}
	}
}
