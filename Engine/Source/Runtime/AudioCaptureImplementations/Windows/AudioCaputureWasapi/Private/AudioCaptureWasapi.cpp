// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureWasapi.h"
#include "WasapiCaptureLog.h"


namespace Audio
{
	FAudioCaptureWasapiStream::FAudioCaptureWasapiStream()
	{
	}

	bool FAudioCaptureWasapiStream::GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex)
	{
		FString DeviceId;
		if (DeviceIndex == DefaultDeviceIndex)
		{
			DeviceId = CaptureDevice.GetDefaultInputDeviceId();
		}
		else
		{
			CaptureDevice.GetDeviceIdFromIndex(DeviceIndex, eAll, DeviceId);
		}

		if (!DeviceId.IsEmpty())
		{
			FWasapiDeviceEnumeration::FDeviceInfo DeviceInfo;

			// DeviceIndex is for collection of all devices here
			if (CaptureDevice.GetDeviceInfo(DeviceId, DeviceInfo))
			{
				FCaptureDeviceInfo CaptureDeviceInfo;
				CaptureDeviceInfo.DeviceId = DeviceInfo.DeviceId;
				CaptureDeviceInfo.DeviceName = DeviceInfo.FriendlyName;
				CaptureDeviceInfo.InputChannels = DeviceInfo.NumInputChannels;
				CaptureDeviceInfo.PreferredSampleRate = DeviceInfo.PreferredSampleRate;
				CaptureDeviceInfo.bSupportsHardwareAEC = false;

				OutInfo = MoveTemp(CaptureDeviceInfo);

				return true;
			}
		}
		else
		{
			UE_LOGF(LogAudioCaptureCore, Display, "FAudioCaptureWasapiStream::GetCaptureDeviceInfo: no default capture device found");
		}

		return false;
	}

	bool FAudioCaptureWasapiStream::OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired)
	{
		if (CaptureDevice.IsStreamOpen())
		{
			CaptureDevice.StopStream();
			CaptureDevice.CloseStream();
		}

		FString DeviceId;
		if (InParams.DeviceIndex == DefaultDeviceIndex)
		{
			DeviceId = CaptureDevice.GetDefaultInputDeviceId();
		}
		else
		{
			// InParams.DeviceIndex is from the capture subset of devices 
			CaptureDevice.GetDeviceIdFromIndex(InParams.DeviceIndex, eCapture, DeviceId);
		}

		if (!DeviceId.IsEmpty())
		{
			FWasapiDeviceEnumeration::FDeviceInfo DeviceInfo;
			if (CaptureDevice.GetDeviceInfo(DeviceId, DeviceInfo))
			{
				FWasapiAudioFormat AudioFormat;
				if (GetAudioFormatFromCaptureParams(InParams, DeviceInfo, AudioFormat))
				{
					FWasapiOnAudioCaptureFunction OnAudioCaptureCallback = [this](void* InBuffer, uint32 InNumFrames, double InStreamPosition, bool bInDiscontinuityError)
					{
						OnAudioCapture(InBuffer, InNumFrames, InStreamPosition, bInDiscontinuityError);
					};

					if (CaptureDevice.OpenStream(DeviceId, AudioFormat, NumFramesDesired, MoveTemp(OnAudioCaptureCallback)))
					{
						NumChannels = AudioFormat.GetNumChannels();
						SampleRate = AudioFormat.GetSampleRate();
						OnCapture = MoveTemp(InOnCapture);

						return true;
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogAudioCaptureCore, Display, "FAudioCaptureWasapiStream::OpenAudioCaptureStream: no default capture device found");
		}

		return false;
	}

	bool FAudioCaptureWasapiStream::CloseStream()
	{
		if (CaptureDevice.IsStreamOpen())
		{
			CaptureDevice.CloseStream();

			return !CaptureDevice.IsStreamOpen();
		}

		return true;
	}

	bool FAudioCaptureWasapiStream::StartStream()
	{
		CaptureDevice.StartStream();

		return CaptureDevice.IsCapturing();
	}

	bool FAudioCaptureWasapiStream::StopStream()
	{
		if (CaptureDevice.IsStreamOpen())
		{
			CaptureDevice.StopStream();

			return !CaptureDevice.IsCapturing();
		}

		return true;
	}

	bool FAudioCaptureWasapiStream::AbortStream()
	{
		if (CaptureDevice.IsStreamOpen())
		{
			CaptureDevice.AbortStream();

			return !CaptureDevice.IsStreamOpen();
		}

		return true;
	}

	bool FAudioCaptureWasapiStream::GetStreamTime(double& OutStreamTime)
	{
		OutStreamTime = CaptureDevice.GetStreamPosition();
		return true;
	}

	bool FAudioCaptureWasapiStream::IsStreamOpen() const
	{
		return CaptureDevice.IsStreamOpen();
	}

	bool FAudioCaptureWasapiStream::IsCapturing() const
	{
		return CaptureDevice.IsCapturing();
	}

	void FAudioCaptureWasapiStream::OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow)
	{
		OnCapture(InBuffer, InBufferFrames, NumChannels, SampleRate, StreamTime, bOverflow);
	}

	bool FAudioCaptureWasapiStream::GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices)
	{
		TArray<FWasapiDeviceEnumeration::FDeviceInfo> Devices;
		if (CaptureDevice.GetInputDevicesAvailable(Devices))
		{
			for (const FWasapiDeviceEnumeration::FDeviceInfo& DeviceInfo : Devices)
			{
				FCaptureDeviceInfo CaptureDeviceInfo;
				CaptureDeviceInfo.DeviceId = DeviceInfo.DeviceId;
				CaptureDeviceInfo.DeviceName = DeviceInfo.FriendlyName;
				CaptureDeviceInfo.InputChannels = DeviceInfo.NumInputChannels;
				CaptureDeviceInfo.PreferredSampleRate = DeviceInfo.PreferredSampleRate;
				CaptureDeviceInfo.bSupportsHardwareAEC = false;

				OutDevices.Emplace(MoveTemp(CaptureDeviceInfo));
			}

			return true;
		}

		return false;
	}

	bool FAudioCaptureWasapiStream::GetAudioFormatFromCaptureParams(
		const FAudioCaptureDeviceParams& InParams, 
		const FWasapiDeviceEnumeration::FDeviceInfo& InDeviceInfo,
		FWasapiAudioFormat& OutAudioFormat)
	{
		// Ignore InParams.NumInputChannels as WASAPI returns AUDCLNT_E_UNSUPPORTED_FORMAT 
		// error for anything other than the device channel count (i.e. if the device 
		// supports 2 channels, specifying 1 channel will error).
		int32 NumChannels = InDeviceInfo.NumInputChannels;
		int32 SampleRate = InParams.SampleRate;

		// If the input params didn't specify sample rate, match the device
		if (SampleRate == InvalidDeviceSampleRate)
		{
			SampleRate = InDeviceInfo.PreferredSampleRate;
		}

		// If the input params didn't specify a bit depth, default to 32-bit float
		EWasapiAudioEncoding AudioEncoding;
		switch (InParams.PCMAudioEncoding)
		{
		case EPCMAudioEncoding::PCM_8:
			AudioEncoding = EWasapiAudioEncoding::PCM_8;
			break;

		case EPCMAudioEncoding::PCM_16:
			AudioEncoding = EWasapiAudioEncoding::PCM_16;
			break;

		case EPCMAudioEncoding::PCM_24:
			AudioEncoding = EWasapiAudioEncoding::PCM_24;
			break;

		case EPCMAudioEncoding::PCM_24_IN_32:
			AudioEncoding = EWasapiAudioEncoding::PCM_24_IN_32;
			break;

		case EPCMAudioEncoding::PCM_32:
			AudioEncoding = EWasapiAudioEncoding::PCM_32;
			break;

		case EPCMAudioEncoding::FLOATING_POINT_32:
			AudioEncoding = EWasapiAudioEncoding::FLOATING_POINT_32;
			break;

		case EPCMAudioEncoding::FLOATING_POINT_64:
			AudioEncoding = EWasapiAudioEncoding::FLOATING_POINT_64;
			break;

		case EPCMAudioEncoding::UNKNOWN:
		default:
			return false;
		}

		OutAudioFormat = FWasapiAudioFormat(NumChannels, SampleRate, AudioEncoding);

		return true;
	}
}
