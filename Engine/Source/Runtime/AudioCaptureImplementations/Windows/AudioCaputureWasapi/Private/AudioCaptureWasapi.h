// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureCore.h"
#include "AudioCaptureDeviceInterface.h"
#include "WasapiStreamManager.h"


namespace Audio
{
	/**
	 * WASAPI implementation of IAudioCaptureStream which encapsulates
	 * capturing audio streams from device.
	 */
	class FAudioCaptureWasapiStream : public IAudioCaptureStream
	{
	public:
		FAudioCaptureWasapiStream();

		// Begin IAudioCaptureStream overrides
		virtual bool GetCaptureDeviceInfo(FCaptureDeviceInfo& OutInfo, int32 DeviceIndex) override;

		virtual bool OpenAudioCaptureStream(const FAudioCaptureDeviceParams& InParams, FOnAudioCaptureFunction InOnCapture, uint32 NumFramesDesired) override;
		
		virtual bool CloseStream() override;

		virtual bool StartStream() override;

		virtual bool StopStream() override;

		virtual bool AbortStream() override;

		virtual bool GetStreamTime(double& OutStreamTime) override;

		virtual int32 GetSampleRate() const override { return SampleRate; }

		virtual bool IsStreamOpen() const override;

		virtual bool IsCapturing() const override;

		virtual void OnAudioCapture(void* InBuffer, uint32 InBufferFrames, double StreamTime, bool bOverflow) override;

		virtual bool GetInputDevicesAvailable(TArray<FCaptureDeviceInfo>& OutDevices) override;
		// End IAudioCaptureStream overrides

	private:
		/** Audio callback which will be called during audio capture. */
		FOnAudioCaptureFunction OnCapture;
		/** The number of channels to capture audio data on from the device. */
		int32 NumChannels = InvalidDeviceChannelCount;
		/** The sample rate used during this capture. */
		int32 SampleRate = InvalidDeviceSampleRate;
		/** The aggregate capture device which manages one or more capture streams. */
		FWasapiStreamManager CaptureDevice;

		/**
		 * GetAudioFormatFromCaptureParams - Converts FAudioCaptureDeviceParams into a corresponding FWasapiAudioFormat
		 * utilizing InDeviceInfo for default values if needed.
		 * 
		 * @param InParams - The capture parameters supplied by the caller to be used when configuring the audio format.
		 * @param InDeviceInfo - The audio device info which will be used as a fallback for default format values if needed.
		 * @param OutAudioFormat - The resulting audio format to corresponding to the input parameters.
		 */
		static bool GetAudioFormatFromCaptureParams(
			const FAudioCaptureDeviceParams& InParams, 
			const FWasapiDeviceEnumeration::FDeviceInfo& InDeviceInfo,
			FWasapiAudioFormat& OutAudioFormat);
	};
}
