// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioCaptureDeviceInterface.h"
#include "AudioCaptureWasapi.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

namespace Audio
{
	
	class FAudioCaptureWasapiFactory : public IAudioCaptureFactory
	{
	public:
		virtual TUniquePtr<IAudioCaptureStream> CreateNewAudioCaptureStream() override
		{
			return TUniquePtr<IAudioCaptureStream>(new FAudioCaptureWasapiStream());
		}
	};

	class FAudioCaptureWasapiModule : public IModuleInterface
	{
	private:
		/** Indicates if FWindowsPlatformMisc::CoInitialize() was successfull. */
		bool bCoInitialized = false;
		
		FAudioCaptureWasapiFactory AudioCaptureFactory;

	public:
		virtual void StartupModule() override
		{
			bCoInitialized = FWindowsPlatformMisc::CoInitialize();
			
			IModularFeatures::Get().RegisterModularFeature(IAudioCaptureFactory::GetModularFeatureName(), &AudioCaptureFactory);
		}

		virtual void ShutdownModule() override
		{
			IModularFeatures::Get().UnregisterModularFeature(IAudioCaptureFactory::GetModularFeatureName(), &AudioCaptureFactory);
			
			if (bCoInitialized)
			{
				FWindowsPlatformMisc::CoUninitialize();
			}
		}
	};
}

IMPLEMENT_MODULE(Audio::FAudioCaptureWasapiModule, AudioCaptureWasapi)
