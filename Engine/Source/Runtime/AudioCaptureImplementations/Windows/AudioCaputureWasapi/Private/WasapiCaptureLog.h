// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureCoreLog.h"

#define WASAPI_CAPTURE_LOG_RESULT(FunctionName, Result) \
	{ \
		FString ErrorString = FString::Printf(TEXT("%s -> 0x%X (line: %d)"), TEXT( FunctionName ), (uint32)Result, __LINE__); \
		UE_LOGF(LogAudioCaptureCore, Error, "WasapiCapture Error: %ls", *ErrorString);				  \
	}
