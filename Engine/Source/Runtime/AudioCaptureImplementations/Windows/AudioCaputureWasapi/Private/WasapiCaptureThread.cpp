// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiCaptureThread.h"
#include "WasapiCaptureLog.h"


namespace Audio
{
	FWasapiCaptureRunnable::FWasapiCaptureRunnable(TSharedPtr<FWasapiInputStream> InStream) :
		InputStream(InStream)
	{
	}

	uint32 FWasapiCaptureRunnable::Run()
	{
		bIsRunning = true;

		bool bCoInitialized = FWindowsPlatformMisc::CoInitialize(ECOMModel::Multithreaded);

		InputStream->StartStream();
		while (bIsRunning.load())
		{
			if (!InputStream->WaitOnBuffer())
			{
				// Accumulate timeouts and report after capture
				++InputStreamTimeoutsDetected;
			}

			if (!InputStream->CaptureAudioFrames())
			{
				// Accumulate device errors which can happen if the device
				// goes off line during capture (e.g. unplugged USB device)
				++InputStreamDeviceErrorsDetected;
			}
		}

		InputStream->StopStream();

		if (bCoInitialized)
		{
			FWindowsPlatformMisc::CoUninitialize();
		}

		return 0;
	}

	void FWasapiCaptureRunnable::Stop()
	{
		bIsRunning = false;
		if (InputStreamTimeoutsDetected > 0)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "WasapiCapture Error: InputStream->WaitOnBuffer() reported %d timeouts during capture", InputStreamTimeoutsDetected);
		}
		if (InputStreamDeviceErrorsDetected > 0)
		{
			UE_LOGF(LogAudioCaptureCore, Error, "WasapiCapture Error: InputStream->WaitOnBuffer() reported %d device during capture", InputStreamDeviceErrorsDetected);
		}
	}

	FWasapiCaptureThread::FWasapiCaptureThread(TSharedPtr<FWasapiInputStream> InStream) :
		CaptureRunnable(MakeUnique<FWasapiCaptureRunnable>(InStream))
	{
	}

	bool FWasapiCaptureThread::Start()
	{
		check(CaptureThread == nullptr);

		CaptureThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(CaptureRunnable.Get(), TEXT("Audio Capture Thread"), 0, TPri_TimeCritical));
		return CaptureThread.IsValid();
	}

	void FWasapiCaptureThread::Stop()
	{
		if (CaptureThread.IsValid())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Audio::FWasapiCaptureThread::Stop);

			bool bShouldWait = true;
			CaptureThread->Kill(bShouldWait);
			CaptureThread = nullptr;
		}
	}

	void FWasapiCaptureThread::Abort()
	{
		if (CaptureThread.IsValid())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Audio::FWasapiCaptureThread::Abort);

			// Always wait for thread to complete otherwise we can crash if
			// the stream is disposed of mid-callback.
			bool bShouldWait = true;
			CaptureThread->Kill(bShouldWait);
			CaptureThread = nullptr;
		}
	}
}
