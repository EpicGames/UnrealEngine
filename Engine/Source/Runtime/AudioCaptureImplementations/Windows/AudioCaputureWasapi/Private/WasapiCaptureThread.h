// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "WasapiInputStream.h"


namespace Audio
{
	/**
	 * FWasapiCaptureRunnable - The runnable which executes the main thread loop for FWasapiCaptureThread.
	 * It has a pointer to a FWasapiInputStream which it uses to coordinating waiting for new data
	 * and delivery of that data to the audio callback.
	 */
	class FWasapiCaptureRunnable : public FRunnable
	{
	public:
		FWasapiCaptureRunnable() = delete;
		FWasapiCaptureRunnable(FWasapiCaptureRunnable&& InOther) = delete;
		FWasapiCaptureRunnable(const FWasapiCaptureRunnable& InOther) = delete;

		FWasapiCaptureRunnable& operator=(FWasapiCaptureRunnable&& InOther) = delete;
		FWasapiCaptureRunnable& operator=(const FWasapiCaptureRunnable& InOther) = delete;

		/**
		 * FWasapiCaptureRunnable - Constructor with given FWasapiInputStream.
		 * 
		 * @param InStream - The stream which will receive audio data as it is received.
		 */
		explicit FWasapiCaptureRunnable(TSharedPtr<FWasapiInputStream> InStream);
		
		/** Default destructor */
		virtual ~FWasapiCaptureRunnable() = default;

		// Begin FRunnable overrides
		virtual uint32 Run() override;
		virtual void Stop() override;
		// End FRunnable overrides

	private:
		/** bIsRunning - The main run loop for this runnable will continue iterating while this flag is true. */
		std::atomic<bool> bIsRunning;
		/**
		 * InputStreamTimeoutsDetected - Accumulates timeouts which occur when the thread event timeout is reached
		 * prior to the event being signaled for new data being available.
		 */
		uint32 InputStreamTimeoutsDetected = 0;
		/**
		 * InputStreamDeviceErrorsDetected - Accumulates any device errors which occur. Device errors which can 
		 * happen if the device goes off line during capture (e.g. unplugged USB device).
		 */
		uint32 InputStreamDeviceErrorsDetected = 0;
		/** InputStream - The input stream which will be run and periodically receive new audio data. */
		TSharedPtr<FWasapiInputStream> InputStream;
	};

	/**
	 * FWasapiCaptureThread - Manages both the FWasapiCaptureRunnable object and the thread whose context it runs in. 
	 */
	class FWasapiCaptureThread
	{
	public:
		FWasapiCaptureThread() = delete;

		/**
		 * FWasapiCaptureThread - Constructor which accepts FWasapiInputStream and creates FWasapiCaptureRunnable with it.
		 */
		explicit FWasapiCaptureThread(TSharedPtr<FWasapiInputStream> InStream);

		/**
		 * Start - Creates the FRunnableThread object which immediately begins running the FWasapiCaptureRunnable member.
		 * 
		 * @return - Boolean indicating of the thread was succesfully created.
		 */
		bool Start();

		/**
		 * Stop - Gracefully shuts down the capture thread.
		 */
		void Stop();

		/**
		 * Abort - Performs non-graceful shutdown of capture thread which will close the underyling thread handle 
		 * without waiting for it to finish.
		 */
		void Abort();

	private:
		/** The thread which is the context that the capture runnable executes in. */
		TUniquePtr<FRunnableThread> CaptureThread;
		/** The capture runnable which manages the run loop for the capture stream. */
		TUniquePtr<FWasapiCaptureRunnable> CaptureRunnable;
	};
}
