// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiDeviceEnumeration.h"

#include "Misc/ScopeExit.h"
#include "WasapiCaptureLog.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

THIRD_PARTY_INCLUDES_START
#include <Functiondiscoverykeys_devpkey.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"


namespace Audio
{
	using FDeviceInfo = FWasapiDeviceEnumeration::FDeviceInfo;

	void FWasapiDeviceEnumeration::Initialize()
	{
		// Assumes FWindowsPlatformMisc::CoInitialize() has been called upstream
		ensure(
			SUCCEEDED(
				::CoCreateInstance(__uuidof(MMDeviceEnumerator), nullptr, CLSCTX_INPROC_SERVER, IID_PPV_ARGS(&DeviceEnumerator))
			) && (DeviceEnumerator != nullptr)
		);

		EnumerateDefaults();
	}

	void FWasapiDeviceEnumeration::EnumerateDefaults()
	{
		auto GetDefaultDeviceID = [this](EDataFlow InDataFlow, ERole InRole, FString& OutDeviceId) -> bool
		{
			// Mark default device.
			bool bSuccess = false;
			TComPtr<IMMDevice> DefaultDevice;
			if (SUCCEEDED(DeviceEnumerator->GetDefaultAudioEndpoint(InDataFlow, InRole, &DefaultDevice)))
			{
				LPWSTR IdString = nullptr;
				if (SUCCEEDED(DefaultDevice->GetId(&IdString)) && IdString)
				{
					OutDeviceId = FString(IdString);
					::CoTaskMemFree(IdString);
					bSuccess = true;
				}
			}
			return bSuccess;
		};

		// Get defaults (render, capture).
		FString DeviceIdName;
		if (GetDefaultDeviceID(eRender, eConsole, DeviceIdName))
		{
			UE_LOGF(LogAudioCaptureCore, Verbose, "FWasapiDeviceEnumeration: Default Device='%ls'", *DeviceIdName);
			DefaultRenderId = DeviceIdName;
		}
		else
		{
			UE_LOGF(LogAudioCaptureCore, Verbose, "FWasapiDeviceEnumeration: no default render device found");
		}

		if (GetDefaultDeviceID(eCapture, eConsole, DeviceIdName))
		{
			UE_LOGF(LogAudioCaptureCore, Verbose, "FWasapiDeviceEnumeration: Default Device='%ls'", *DeviceIdName);
			DefaultCaptureId = DeviceIdName;
		}
		else
		{
			UE_LOGF(LogAudioCaptureCore, Verbose, "FWasapiDeviceEnumeration: no default capture device found");
		}
	}

	FString FWasapiDeviceEnumeration::GetDefaultInputDeviceId()
	{
		return DefaultCaptureId;
	}

	FString FWasapiDeviceEnumeration::GetDefaultOutputDeviceId()
	{
		return DefaultRenderId;
	}

	bool FWasapiDeviceEnumeration::GetDeviceInfo(const FString& InDeviceId, FDeviceInfo& OutDeviceInfo)
	{
		TComPtr<IMMDevice> Device;
		if (GetIMMDevice(InDeviceId, Device))
		{
			FDeviceInfo DeviceInfo;
			if (GetDeviceProperties(Device, DeviceInfo))
			{
				OutDeviceInfo = MoveTemp(DeviceInfo);
				return true;
			}
		}

		return false;
	}

	bool FWasapiDeviceEnumeration::GetDeviceIdFromIndex(int32 InDeviceIndex, EDataFlow InDataFlow, FString& OutDeviceId)
	{
		TComPtr<IMMDeviceCollection> Collection;
		HRESULT Result = DeviceEnumerator->EnumAudioEndpoints(InDataFlow, DEVICE_STATE_ACTIVE, &Collection);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDeviceCollection::EnumAudioEndpoints", Result);
			return false;
		}

		TComPtr<IMMDevice> Device;
		Result = Collection->Item(InDeviceIndex, &Device);
		if (SUCCEEDED(Result))
		{
			LPWSTR IdString = nullptr;
			if (SUCCEEDED(Device->GetId(&IdString)) && IdString)
			{
				OutDeviceId = FString(IdString);
				::CoTaskMemFree(IdString);
				return true;
			}
		}

		return false;
	}

	bool FWasapiDeviceEnumeration::GetDeviceIndexFromId(const FString& InDeviceId, int32& OutDeviceIndex)
	{
		OutDeviceIndex = INDEX_NONE;

		TComPtr<IMMDeviceCollection> Collection;
		// For compatibility with the RtAudio implementation, we get all device types here
		HRESULT Result = DeviceEnumerator->EnumAudioEndpoints(eAll, DEVICE_STATE_ACTIVE, &Collection);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDeviceCollection::EnumAudioEndpoints", Result);
			return false;
		}

		uint32 Count;
		Result = Collection->GetCount(&Count);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDeviceCollection::GetCount", Result);
			return false;
		}

		for (uint32 DeviceIndex = 0; DeviceIndex < Count; ++DeviceIndex)
		{
			TComPtr<IMMDevice> Device;
			Result = Collection->Item(DeviceIndex, &Device);
			if (SUCCEEDED(Result))
			{
				FString DeviceId;
				if (GetDeviceId(Device, DeviceId))
				{
					if (DeviceId == InDeviceId)
					{
						OutDeviceIndex = DeviceIndex;
						return true;
					}
				}
			}
		}

		return false;
	}

	bool FWasapiDeviceEnumeration::GetIMMDevice(const FString& InDeviceId, TComPtr<IMMDevice>& OutDevice)
	{
		TComPtr<IMMDevice> Device;
		HRESULT Result = DeviceEnumerator->GetDevice(TCHAR_TO_WCHAR(*InDeviceId), &Device);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDeviceEnumerator::GetDevice", Result);
			return false;
		}

		OutDevice = Device;
		return true;
	}

	bool FWasapiDeviceEnumeration::GetInputDevicesAvailable(TArray<FDeviceInfo>& OutDevices)
	{
		TComPtr<IMMDeviceCollection> Collection;
		HRESULT Result = DeviceEnumerator->EnumAudioEndpoints(eCapture, DEVICE_STATE_ACTIVE, &Collection);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDeviceCollection::EnumAudioEndpoints", Result);
			return false;
		}

		uint32 Count;
		Result = Collection->GetCount(&Count);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDeviceCollection::GetCount", Result);
			return false;
		}

		for (uint32 DeviceIndex = 0; DeviceIndex < Count; ++DeviceIndex)
		{
			TComPtr<IMMDevice> Device;
			Result = Collection->Item(DeviceIndex, &Device);
			if (SUCCEEDED(Result))
			{
				FDeviceInfo DeviceInfo;
				if (GetDeviceProperties(Device, DeviceInfo))
				{
					UE_LOGF(LogAudioCaptureCore, Verbose, "Device %d: \"%ls\" (%ls)", DeviceIndex,
						DeviceInfo.FriendlyName.GetCharArray().GetData(), DeviceInfo.DeviceId.GetCharArray().GetData());

					UE_LOGF(LogAudioCaptureCore, Verbose, "\tBPS: %d, SR: %d", DeviceInfo.BitsPerSample, DeviceInfo.PreferredSampleRate);

					OutDevices.Emplace(MoveTemp(DeviceInfo));
				}
				else
				{
					UE_LOGF(LogAudioCaptureCore, Warning, "Unable to fetch audio capture device properties...skipping");
				}
			}
			else
			{
				UE_LOGF(LogAudioCaptureCore, Warning, "Unable to fetch audio capture device...skipping");
			}
		}

		return true;
	}

	bool FWasapiDeviceEnumeration::GetDeviceProperties(TComPtr<IMMDevice> InDevice, FDeviceInfo& OutInfo)
	{
		FString IdString;
		FString FriendlyName;
		if (GetDeviceId(InDevice, IdString) && GetDeviceFriendlyName(InDevice, FriendlyName))
		{
			TComPtr<IAudioClient3> AudioClient;
			HRESULT Result = InDevice->Activate(__uuidof(IAudioClient3), CLSCTX_INPROC_SERVER, nullptr, IID_PPV_ARGS_Helper(&AudioClient));
			if (FAILED(Result))
			{
				WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::Activate", Result);
				return false;
			}

			WAVEFORMATEX* DeviceFormat = nullptr;
			if (GetAudioClientMixFormat(AudioClient, &DeviceFormat))
			{
				ON_SCOPE_EXIT{ ::CoTaskMemFree(DeviceFormat); };

				OutInfo.DeviceId = IdString;
				OutInfo.FriendlyName = FriendlyName;
				OutInfo.BitsPerSample = DeviceFormat->wBitsPerSample;
				OutInfo.PreferredSampleRate = DeviceFormat->nSamplesPerSec;

				OutInfo.EndpointType = GetEndpointType(InDevice);
				if (OutInfo.EndpointType == FDeviceInfo::EEndpointType::Capture)
				{
					OutInfo.NumInputChannels = DeviceFormat->nChannels;
				}
				else if (OutInfo.EndpointType == FDeviceInfo::EEndpointType::Render)
				{
					OutInfo.NumOutputChannels = DeviceFormat->nChannels;
				}

				return true;
			}
		}

		return false;
	}

	bool FWasapiDeviceEnumeration::GetDeviceId(TComPtr<IMMDevice> InDevice, FString& OutString)
	{
		LPWSTR IdString = nullptr;
		HRESULT Result = InDevice->GetId(&IdString);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDevice::GetId", Result);
			return false;
		}

		OutString = FString(IdString);
		::CoTaskMemFree(IdString);

		return true;
	}

	bool FWasapiDeviceEnumeration::GetDeviceFriendlyName(TComPtr<IMMDevice> InDevice, FString& OutString)
	{
		TComPtr<IPropertyStore> Props;
		HRESULT Result = InDevice->OpenPropertyStore(STGM_READ, &Props);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDevice::OpenPropertyStore", Result);
			return false;
		}

		PROPVARIANT VarName;
		// Initialize container for property value.
		::PropVariantInit(&VarName);

		// Get the endpoint's friendly-name property.
		Result = Props->GetValue(PKEY_Device_FriendlyName, &VarName);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IPropertyStore::GetValue", Result);
			return false;
		}

		OutString = FString(VarName.pwszVal);
		::PropVariantClear(&VarName);

		return true;
	}

	FDeviceInfo::EEndpointType FWasapiDeviceEnumeration::GetEndpointType(TComPtr<IMMDevice> InDevice)
	{
		TComPtr<IMMEndpoint> Endpoint;
		HRESULT Result = InDevice->QueryInterface(IID_PPV_ARGS(&Endpoint));
		if (SUCCEEDED(Result))
		{
			EDataFlow DataFlow = eRender;
			Result = Endpoint->GetDataFlow(&DataFlow);
			if (SUCCEEDED(Result))
			{
				switch (DataFlow)
				{
				case eRender:
					return FDeviceInfo::EEndpointType::Render;
				case eCapture:
					return FDeviceInfo::EEndpointType::Capture;
				default:
					break;
				}
			}
			else
			{
				WASAPI_CAPTURE_LOG_RESULT("IMMEndpoint::GetDataFlow", Result);
			}
		}
		else
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDevice::QueryInterface", Result);
		}

		return FDeviceInfo::EEndpointType::Unknown;
	}

	bool FWasapiDeviceEnumeration::GetAudioClientMixFormat(TComPtr<IAudioClient3> AudioClient, WAVEFORMATEX** OutFormat)
	{
		HRESULT Result = AudioClient->GetMixFormat(OutFormat);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::GetMixFormat", Result);
			return false;
		}

		return true;
	}
}