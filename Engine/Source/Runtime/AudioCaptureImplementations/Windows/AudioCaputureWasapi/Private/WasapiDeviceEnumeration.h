// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureCore.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <AudioClient.h>
#include <Mmdeviceapi.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"


namespace Audio
{
	/**
	 * FWasapiDeviceEnumeration - Class which wraps Microsoft's IMM device enumeration API.
	 */
	class FWasapiDeviceEnumeration
	{
	public:
		/**
		 * FDeviceInfo - Properties associated with an audio device.
		 */
		struct FDeviceInfo
		{
			/**
			 * EEndpointType - Designates whether a device is a capture or render device.
			 */
			enum class EEndpointType { Unknown, Render, Capture };

			/** Unique identifier for this device. */
			FString DeviceId;
			/** Human readable name for this device. */
			FString FriendlyName;
			/** Number of input channels on this device available for recording. */
			uint32 NumInputChannels = 0;
			/** Number of output channels on this device available for playback. */
			uint32 NumOutputChannels = 0;
			/** The current bit depth configured for this device. Additional bit depths may be supported. */
			uint32 BitsPerSample = 0;
			/** The preferred sample rate for this device. Additional sample rates may be supported. */
			uint32 PreferredSampleRate = 0;
			/** The endpoint type for this device (Render or Capture). */
			EEndpointType EndpointType = EEndpointType::Unknown;
		};

		FWasapiDeviceEnumeration() = default;
		~FWasapiDeviceEnumeration() = default;

		/** Instantiates DeviceEnumerator and initializes the default input and output devices IDs.  */
		void Initialize();

		/** Returns the Id of the default input device. */
		FString GetDefaultInputDeviceId();
		/** Returns the Id of the default output device. */
		FString GetDefaultOutputDeviceId();

		/**
		 * GetDeviceInfo - Gets the FDeviceInfo for the device with the given Id.
		 * 
		 * @param InDeviceId - The Id of the device to fetch information for.
		 * @param OutDeviceInfo - Upon success, the device info struct for the device. 
		 * Untouched on failure.
		 * @return - Boolean indicating success or failure when attempting to fetch the device info.
		 */
		bool GetDeviceInfo(const FString& InDeviceId, FDeviceInfo& OutDeviceInfo);

		/**
		 * GetDeviceIdFromIndex - Gets the device Id from a device index.
		 * 
		 * @param InDeviceIndex - The index for the device to get the Id for.
		 * @param InDataFlow - Indicates which device collection to search (render, capture or both).
		 * @param OutDeviceId - Upon success, the Id of the device at the given index.
		 * @return - Boolean indicating success or failure when looking up the device index.
		 */
		bool GetDeviceIdFromIndex(int32 InDeviceIndex, EDataFlow InDataFlow, FString& OutDeviceId);
		
		/**
		 * GetDeviceIndexFromId - Gets the index of a device with the given Id. Searches the
		 * collection of active devices for both capture and render data flows. 
		 * 
		 * @param InDeviceId - Device Id to get the index for.
		 * @param OutDeviceIndex - Upon success, will contain the index for the device.
		 * @return - Boolean indicating success or failure when looking up the device Id.
		 */
		bool GetDeviceIndexFromId(const FString& InDeviceId, int32& OutDeviceIndex);

		/**
		 * GetInputDevicesAvailable - Gets the collection of currently active audio capture devices.
		 * 
		 * @param OutDevices - TArray of FDeviceInfo objects for the currently available capture devices.
		 * @return - Boolean indicating success or failure when access the collection of devices.
		 */
		bool GetInputDevicesAvailable(TArray<FDeviceInfo>& OutDevices);

		/**
		 * GetIMMDevice - Gets a COM pointer to the IMMDevice for the give device Id.
		 * 
		 * @param InDeviceId - Device Id of the device to get.
		 * @param OutDevice - COM pointer to the device for the given Id.
		 * @return - Boolean indicating success or failure when looking up the give Id.
		 */
		bool GetIMMDevice(const FString& InDeviceId, TComPtr<IMMDevice>& OutDevice);

	private:
		/** The Id of the system default render device. */
		FString DefaultRenderId;
		/** The Id of the system default capture device. */
		FString DefaultCaptureId;
		/** COM pointer to the device enumerator used by this class to find audio devices and their properties. */
		TComPtr<IMMDeviceEnumerator> DeviceEnumerator;

		/** EnumerateDefaults - Get the IDs for the system default devices. */
		void EnumerateDefaults();

		/** GetDeviceProperties - Convenience function for getting the device info of an IMMDevice. */
		bool GetDeviceProperties(TComPtr<IMMDevice> InDevice, FDeviceInfo& OutInfo);

		/** GetDeviceId - Convenience function for getting the device Id of an IMMDevice. */
		bool GetDeviceId(TComPtr<IMMDevice> InDevice, FString& OutString);

		/** GetDeviceFriendlyName - Convenience function for getting the friendly name of an IMMDevice. */
		bool GetDeviceFriendlyName(TComPtr<IMMDevice> InDevice, FString& OutString);

		/** GetEndpointType - Returns endpoint type (render or capture) for the given IMMDevice. */
		FDeviceInfo::EEndpointType GetEndpointType(TComPtr<IMMDevice> InDevice);

		/**
		 * GetAudioClientMixFormat - Returns the current mix format for the given audio client.
		 * The mix format is the audio format used by the OS audio mixer when mixing shared
		 * mode streams.
		 */
		bool GetAudioClientMixFormat(TComPtr<IAudioClient3> AudioClient, WAVEFORMATEX** OutFormat);
	};

}
