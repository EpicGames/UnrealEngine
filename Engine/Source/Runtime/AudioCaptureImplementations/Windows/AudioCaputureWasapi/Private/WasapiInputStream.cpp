// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiInputStream.h"

#include "WasapiCaptureLog.h"

namespace Audio
{
	// REFERENCE_TIME base which is in units of 100 nanoseconds
	static constexpr REFERENCE_TIME ReferenceTimeBase = 1e7;

	FWasapiInputStream::FWasapiInputStream(
		TComPtr<IMMDevice> InDevice, 
		const FWasapiAudioFormat& InFormat, 
		uint32 InNumFramesDesired,
		FWasapiOnAudioCaptureFunction InCallback)
	{
		check(InDevice.IsValid());

		// Activating the device, which gives us the audio client.
		TComPtr<IAudioClient3> TempAudioClient;
		HRESULT Result = InDevice->Activate(__uuidof(IAudioClient3), CLSCTX_INPROC_SERVER, nullptr, IID_PPV_ARGS_Helper(&TempAudioClient));
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IMMDevice::Activate", Result);
			return;
		}

		WAVEFORMATEX* MixFormat = nullptr;
		Result = TempAudioClient->GetMixFormat(&MixFormat);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::GetMixFormat", Result);
			return;
		}

		REFERENCE_TIME MinimumPeriodInRefTimes = 0;
		Result = TempAudioClient->GetDevicePeriod(nullptr, &MinimumPeriodInRefTimes);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::GetDevicePeriod", Result);
			return;
		}

		// Audio events will be delivered to us rather than needing to poll
		uint32 Flags = AUDCLNT_STREAMFLAGS_EVENTCALLBACK;
		// Convert desired buffer size from frames to REFERENCE_TIME which is in units of 100 nanoseconds
		double BufferDurationSeconds = static_cast<double>(InNumFramesDesired) / InFormat.GetSampleRate();
		REFERENCE_TIME DesiredBufferDuration = static_cast<REFERENCE_TIME>(BufferDurationSeconds * ReferenceTimeBase);
		// Ensure the requested duration is at least as large as the minimum supported by this device
		DesiredBufferDuration = FMath::Max(MinimumPeriodInRefTimes, DesiredBufferDuration);

		Result = TempAudioClient->Initialize(AUDCLNT_SHAREMODE_SHARED, Flags, DesiredBufferDuration, 0, InFormat.GetWaveFormat(), nullptr);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::Initialize", Result);
			return;
		}

		Result = TempAudioClient->GetBufferSize(&NumFramesPerBuffer);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::GetBufferSize", Result);
			return;
		}

		// Not using FEvent/FEventWin here because we need access to the raw, platform
		// handle (see SetEventHandler() below).
		EventHandle = ::CreateEvent(nullptr, 0, 0, nullptr);
		if (EventHandle == nullptr)
		{
			WASAPI_CAPTURE_LOG_RESULT("CreateEvent", Result);
			return;
		}

		Result = TempAudioClient->SetEventHandle(EventHandle);
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::SetEventHandle", Result);
			return;
		}

		TComPtr<IAudioCaptureClient> TempCaptureClient;
		Result = TempAudioClient->GetService(__uuidof(IAudioCaptureClient), IID_PPV_ARGS_Helper(&TempCaptureClient));
		if (FAILED(Result))
		{
			WASAPI_CAPTURE_LOG_RESULT("IAudioClient3::GetService", Result);
			return;
		}

		AudioClient = MoveTemp(TempAudioClient);
		CaptureClient = MoveTemp(TempCaptureClient);
		AudioFormat = InFormat;
		OnAudioCaptureCallback = MoveTemp(InCallback);
		SilienceBuffer.SetNumZeroed(GetBufferSizeBytes(), EAllowShrinking::No);

		bIsInitialized = true;

		UE_LOGF(LogAudioCaptureCore, Display, "WasapiCapture AudioFormat SampeRate: %d, BitDepth: %ls", 
			AudioFormat.GetSampleRate(), *(AudioFormat.GetEncodingString()));
	}

	FWasapiInputStream::~FWasapiInputStream()
	{
		if (EventHandle)
		{
			::CloseHandle(EventHandle);
		}
	}

	bool FWasapiInputStream::IsInitialized() const
	{
		return bIsInitialized;
	}

	void FWasapiInputStream::StartStream()
	{
		if (AudioClient)
		{
			AudioClient->Start();

			// Drain the CaptureClient input buffer
			// see comment here for more info: 
			// https://learn.microsoft.com/en-us/windows/win32/api/audioclient/nf-audioclient-iaudiocaptureclient-getbuffer
			DrainInputBuffer();
		}
	}

	void FWasapiInputStream::StopStream()
	{ 
		if (AudioClient)
		{
			AudioClient->Stop();
		}
	}

	bool FWasapiInputStream::WaitOnBuffer() const
	{
		static constexpr uint32 TimeoutInMs = 1000;

		if (::WaitForSingleObject(EventHandle, TimeoutInMs) != WAIT_OBJECT_0)
		{
			return false;
		}

		return true;
	}

	uint32 FWasapiInputStream::GetBufferSizeFrames() const
	{
		return NumFramesPerBuffer;
	}

	uint32 FWasapiInputStream::GetBufferSizeBytes() const
	{
		return NumFramesPerBuffer * AudioFormat.GetNumChannels() * AudioFormat.GetBytesPerSample();
	}

	double FWasapiInputStream::GetStreamPosition() const
	{
		return DevicePosition;
	}

	bool FWasapiInputStream::DrainInputBuffer()
	{
		uint32 NextPacketSize = 0;
		HRESULT Result = CaptureClient->GetNextPacketSize(&NextPacketSize);
		if (FAILED(Result))
		{
			// Don't log from the audio thread.
			return false;
		}

		while (NextPacketSize > 0)
		{
			DWORD Flags = 0;
			uint8* Data = nullptr;
			uint32 NumFramesRead = 0;

			Result = CaptureClient->GetBuffer(&Data, &NumFramesRead, &Flags, nullptr, nullptr);
			if (FAILED(Result))
			{
				return false;
			}

			CaptureClient->ReleaseBuffer(NumFramesRead);

			Result = CaptureClient->GetNextPacketSize(&NextPacketSize);
			if (FAILED(Result))
			{
				return false;
			}
		}

		return true;
	}

	bool FWasapiInputStream::CaptureAudioFrames()
	{
		SCOPED_NAMED_EVENT(FWasapiInputStream_CaptureAudioFrames, FColor::Red);

		uint32 NextPacketSize = 0;
		HRESULT Result = CaptureClient->GetNextPacketSize(&NextPacketSize);
		if (FAILED(Result))
		{
			// We aren't logging failures here because this method is called in the
			// audio thread context.
			return false;
		}

		DWORD Flags = 0;
		uint8* Data = nullptr;
		uint32 NumFramesRead = 0;
		uint64 TempDevicePosition = 0;

		while (NextPacketSize > 0)
		{
			Result = CaptureClient->GetBuffer(&Data, &NumFramesRead, &Flags, &TempDevicePosition, nullptr);
			if (FAILED(Result))
			{
				return false;
			}

			DevicePosition.store(static_cast<double>(TempDevicePosition) / AudioFormat.GetSampleRate(), std::memory_order_relaxed);
			bool bHasDiscontinuityError = ((Flags & AUDCLNT_BUFFERFLAGS_DATA_DISCONTINUITY) != 0);

			// If this flag is set we need to ignore the data returned by GetBuffer()
			// and treat it as silence. We utilize SilienceBuffer for this.
			if ((Flags & AUDCLNT_BUFFERFLAGS_SILENT) != 0)
			{
				int32 BytesRead = NumFramesRead * AudioFormat.GetNumChannels() * AudioFormat.GetBytesPerSample();
				check(SilienceBuffer.Max() >= BytesRead);
				Data = SilienceBuffer.GetData();
			}

			if (OnAudioCaptureCallback != nullptr)
			{
				OnAudioCaptureCallback(Data, NumFramesRead, DevicePosition, bHasDiscontinuityError);
			}

			CaptureClient->ReleaseBuffer(NumFramesRead);

			Result = CaptureClient->GetNextPacketSize(&NextPacketSize);
			if (FAILED(Result))
			{
				return false;
			}
		}

		return true;
	}
}
