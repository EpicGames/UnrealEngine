// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureCore.h"
#include "WasapiAudioFormat.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

#include "Microsoft/COMPointer.h"

THIRD_PARTY_INCLUDES_START
#include <AudioClient.h>
#include <comdef.h>
#include <Mmdeviceapi.h>
THIRD_PARTY_INCLUDES_END

#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"


namespace Audio
{
	/**
	 * FWasapiOnAudioCaptureFunction - Callback function type for receiving audio from a capture device.
	 */
	typedef TFunction<void(void* InBuffer, uint32 InNumFrames, double InStreamPosition, bool bInDiscontinuityError)> FWasapiOnAudioCaptureFunction;

	/**
	 * FWasapiInputStream - This class manages the WASPI audio client, capture buffers and audio callback used during capture.
	 */
	class FWasapiInputStream
	{
	public:
		FWasapiInputStream() = delete;
		FWasapiInputStream(FWasapiInputStream&& InOther) = delete;
		FWasapiInputStream(const FWasapiInputStream& InOther) = delete;

		FWasapiInputStream(
			TComPtr<IMMDevice> InDevice, 
			const FWasapiAudioFormat& InFormat, 
			uint32 InNumFramesDesired,
			FWasapiOnAudioCaptureFunction InCallback);

		~FWasapiInputStream();

		FWasapiInputStream& operator=(FWasapiInputStream&& InOther) = delete;
		FWasapiInputStream& operator=(const FWasapiInputStream& InOther) = delete;

		/** Returns whether or not this object has been successfully initialized. */
		bool IsInitialized() const;

		/** Starts the audio capture which triggers periodic callbacks with new audio data. */
		void StartStream();
		/** Stops the audio client which will shut down the stream and any further callbacks. */
		void StopStream();

		/** Waits on the audio client event handle which will get set when new capture data is available. */
		bool WaitOnBuffer() const;
		/** The capture thread calls this periodically with new audio data during a capture. */
		bool CaptureAudioFrames();

		/** Returns the buffer size in frames for the buffer for the audio callback. */
		uint32 GetBufferSizeFrames() const;
		/** Returns the buffer size in bytes corresponding to the buffer for the audio callback. */
		uint32 GetBufferSizeBytes() const;
		/** Returns the current stream position in seconds. */
		double GetStreamPosition() const;

	private:

		/** COM pointer to the WASAPI audio client object. */
		TComPtr<IAudioClient3> AudioClient;
		/** COM pointer to the WASAPI capture client object. */
		TComPtr<IAudioCaptureClient> CaptureClient;

		/** Holds the audio format configuration for this stream. */
		FWasapiAudioFormat AudioFormat;

		/** Indicates if this object has been successfully initialized. */
		bool bIsInitialized = false;

		/** Number of frames of audio data which will be used for each audio callback during capture. */
		uint32 NumFramesPerBuffer = 0;
		/** Current device position as reported by WASAPI each callback. */
		std::atomic<double> DevicePosition = 0.0;
		/**
		 * Event handle which our capture thread waits on prior to each callback. WASAPI signals this 
		 * object each quanta when a buffer of audio has been captured and is ready to be consumed downstream.
		 */
		HANDLE EventHandle = nullptr;

		/** The callback used to periodically deliver new audio data during capture. */
		FWasapiOnAudioCaptureFunction OnAudioCaptureCallback;
		/** Buffer used when WASAPI indicates that silence should be output for a given callback. */
		TArray<uint8> SilienceBuffer;

		/**
		 * DrainInputBuffer - Prepares the capture client by draining the input buffer of any audio data
		 * which may have accumulated.
		 */
		bool DrainInputBuffer();
	};
}
