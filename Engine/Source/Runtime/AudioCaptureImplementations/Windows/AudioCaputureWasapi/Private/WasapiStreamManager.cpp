// Copyright Epic Games, Inc. All Rights Reserved.

#include "WasapiStreamManager.h"

#include "AudioCaptureCoreLog.h"
#include "Misc/ScopeExit.h"


namespace Audio
{
	FWasapiStreamManager::FWasapiStreamManager()
	{
		DeviceEnumerator.Initialize();
	}

	FWasapiStreamManager::~FWasapiStreamManager()
	{
		CloseStream();
	}

	FString FWasapiStreamManager::GetDefaultInputDeviceId()
	{
		return DeviceEnumerator.GetDefaultInputDeviceId();
	}

	FString FWasapiStreamManager::GetDefaultOutputDeviceId()
	{
		return DeviceEnumerator.GetDefaultOutputDeviceId();
	}

	bool FWasapiStreamManager::GetDeviceInfo(const FString& InDeviceId, FDeviceInfo& OutDeviceInfo)
	{
		return DeviceEnumerator.GetDeviceInfo(InDeviceId, OutDeviceInfo);
	}

	bool FWasapiStreamManager::GetDeviceIndexFromId(const FString& InDeviceId, int32& OutDeviceIndex)
	{
		return DeviceEnumerator.GetDeviceIndexFromId(InDeviceId, OutDeviceIndex);
	}

	bool FWasapiStreamManager::GetDeviceIdFromIndex(int32 InDeviceIndex, EDataFlow InDataFlow, FString& OutDeviceId)
	{
		return DeviceEnumerator.GetDeviceIdFromIndex(InDeviceIndex, InDataFlow, OutDeviceId);
	}

	bool FWasapiStreamManager::GetInputDevicesAvailable(TArray<FDeviceInfo>& OutDevices)
	{
		return DeviceEnumerator.GetInputDevicesAvailable(OutDevices);
	}

	bool FWasapiStreamManager::OpenStream(
		const FString& InDeviceId, 
		const FWasapiAudioFormat& InFormat,
		uint32 InNumFramesDesired,
		FWasapiOnAudioCaptureFunction InCallback)
	{
		check(State == EStreamState::STREAM_CLOSED);
		check(InputStream == nullptr);

		TComPtr<IMMDevice> Device;
		if (DeviceEnumerator.GetIMMDevice(InDeviceId, Device))
		{
			InputStream = MakeShared<FWasapiInputStream>(Device, InFormat, InNumFramesDesired, MoveTemp(InCallback));
			if (InputStream->IsInitialized())
			{
				CaptureThread = MakeUnique<FWasapiCaptureThread>(InputStream);
				State = EStreamState::STREAM_STOPPED;

				return true;
			}
			else
			{
				InputStream = nullptr;
			}
		}

		return false;
	}

	bool FWasapiStreamManager::IsStreamOpen() const
	{
		return State != EStreamState::STREAM_CLOSED;
	}

	void FWasapiStreamManager::StartStream()
	{
		check(State == EStreamState::STREAM_STOPPED);

		if (CaptureThread->Start())
		{
			State = EStreamState::STREAM_CAPTURING;
		}
		else
		{
			State = EStreamState::INVALID_STATE;
		}
	}

	bool FWasapiStreamManager::IsCapturing() const
	{
		return State == EStreamState::STREAM_CAPTURING;
	}

	void FWasapiStreamManager::StopStream()
	{
		check(State == EStreamState::STREAM_CAPTURING);

		State = EStreamState::STREAM_STOPPING;
		CaptureThread->Stop();
		State = EStreamState::STREAM_STOPPED;
	}

	void FWasapiStreamManager::AbortStream()
	{
		State = EStreamState::STREAM_STOPPING;
		CaptureThread->Abort();
		State = EStreamState::STREAM_STOPPED;
	}

	void FWasapiStreamManager::CloseStream()
	{
		if (State == EStreamState::STREAM_CAPTURING)
		{
			StopStream();
		}

		CaptureThread = nullptr;
		InputStream = nullptr;

		State = EStreamState::STREAM_CLOSED;
	}

	uint32 FWasapiStreamManager::GetStreamBufferSizeBytes() const
	{
		return InputStream->GetBufferSizeBytes();
	}

	double FWasapiStreamManager::GetStreamPosition() const
	{
		return InputStream->GetStreamPosition();
	}
}
