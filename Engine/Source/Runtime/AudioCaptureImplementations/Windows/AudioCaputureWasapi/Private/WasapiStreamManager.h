// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioCaptureCore.h"
#include "WasapiDeviceEnumeration.h"
#include "WasapiInputStream.h"
#include "WasapiCaptureThread.h"


namespace Audio
{
	/**
	 * FWasapiStreamManager - Top level object for managing audio captures. It is composed of
	 * a device enumerator, capture thread, input stream and state machine. In the future, 
	 * additional input streams may be managed here in support of multi-track capture.
	 */
	class FWasapiStreamManager
	{
	public:

		using FDeviceInfo = FWasapiDeviceEnumeration::FDeviceInfo;

		FWasapiStreamManager();
		FWasapiStreamManager(FWasapiStreamManager&& InOther) = delete;
		FWasapiStreamManager(const FWasapiStreamManager& InOther) = delete;

		virtual ~FWasapiStreamManager();

		FWasapiStreamManager& operator=(FWasapiStreamManager&& InOther) = delete;
		FWasapiStreamManager& operator=(const FWasapiStreamManager& InOther) = delete;

		/** Returns the default input device Id. */
		FString GetDefaultInputDeviceId();
		/** Returns the default output device Id. */
		FString GetDefaultOutputDeviceId();

		/**
		 * GetDeviceInfo - Convienence method for FWasapiDeviceEnumeration::GetDeviceInfo()
		 */
		bool GetDeviceInfo(const FString& InDeviceId, FDeviceInfo& OutDeviceInfo);

		/**
		 * GetDeviceIndexFromId - Convienence method for FWasapiDeviceEnumeration::GetDeviceIndexFromId()
		 */
		bool GetDeviceIndexFromId(const FString& InDeviceId, int32& OutDeviceIndex);

		/**
		 * GetDeviceIdFromIndex - Convienence method for FWasapiDeviceEnumeration::GetDeviceIdFromIndex()
		 */
		bool GetDeviceIdFromIndex(int32 InDeviceIndex, EDataFlow InDataFlow, FString& OutDeviceId);

		/**
		 * GetInputDevicesAvailable - Convienence method for FWasapiDeviceEnumeration::GetInputDevicesAvailable()
		 */
		bool GetInputDevicesAvailable(TArray<FDeviceInfo>& OutDevices);

		/**
		 * OpenStream - Finds the device for the given Id and prepares it for capturing audio. It also
		 * spawns a capture thread.
		 * 
		 * @param InDeviceId - Id of the device to use for capturing audio.
		 * @param InFormat - The audio format to use for the capture.
		 * @param InNumFramesDesired - The number of audio frames desired when audio is delivered via 
		 * the capture callback.
		 * @param InCallback - The callback which is called periodically during capture with audio data.
		 */
		bool OpenStream(
			const FString& InDeviceId, 
			const FWasapiAudioFormat& InFormat,
			uint32 InNumFramesDesired,
			FWasapiOnAudioCaptureFunction InCallback);

		/** Returns whether or not this object has been successfully initialized. */
		bool IsStreamOpen() const;

		/** Starts the audio capture which triggers periodic callbacks with new audio data. */
		void StartStream();
		/** Indicates if the stream was successfully started and is capturing data. */
		bool IsCapturing() const;
		/** Stops the capture thread which, in turn, completes the runnable and stops the input stream. */
		void StopStream();

		/** Brute force stops the capture thread. */
		void AbortStream();
		/** Releases resources related to the capture thread and input streams. */
		void CloseStream();

		/** Returns the buffer size in bytes of the callback buffer. */
		uint32 GetStreamBufferSizeBytes() const;
		/** Returns the current position (in seconds) of the input stream. */
		double GetStreamPosition() const;

	private:
		/** The states used by the state machine. */
		enum class EStreamState {
			INVALID_STATE,
			STREAM_CLOSED,
			STREAM_STOPPED,
			STREAM_STOPPING,
			STREAM_CAPTURING
		};

		/** Current state of the state machine. */
		EStreamState State = EStreamState::STREAM_CLOSED;

		/** Object used to enumerate audio devices present in the system. */
		FWasapiDeviceEnumeration DeviceEnumerator;

		/** The input stream for the selected device which will stream audio during capture. */
		TSharedPtr<FWasapiInputStream> InputStream;

		/** The thread which provides an execution context during audio capture. */
		TUniquePtr<FWasapiCaptureThread> CaptureThread;
	};
}
